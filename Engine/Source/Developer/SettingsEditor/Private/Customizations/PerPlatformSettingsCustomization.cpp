// Copyright Epic Games, Inc. All Rights Reserved.

#include "PerPlatformSettingsCustomization.h"
#include "DetailWidgetRow.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Images/SImage.h"
#include "Styling/AppStyle.h"
#include "Engine/PlatformSettings.h"
#include "IDetailGroup.h"
#include "IDetailChildrenBuilder.h"

TSharedRef<IPropertyTypeCustomization> FPerPlatformSettingsCustomization::MakeInstance()
{
	return MakeShareable(new FPerPlatformSettingsCustomization);
}

FPerPlatformSettingsCustomization::FPerPlatformSettingsCustomization()
{
}

void FPerPlatformSettingsCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{

	static const FName ShowOnlyInners("ShowOnlyInnerProperties");
	const bool bShowHeader = !PropertyHandle->HasMetaData(ShowOnlyInners);
	if (bShowHeader)
	{
		HeaderRow
			.WholeRowContent()
			.MaxDesiredWidth(TOptional<float>())
			.MaxDesiredWidth(TOptional<float>())
			[
				PropertyHandle->CreatePropertyNameWidget()
			];
	}
}

void FPerPlatformSettingsCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	TSharedPtr<IPropertyHandle> SettingsProperty = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FPerPlatformSettings, Settings));
	TSharedPtr<IPropertyHandleArray> SettingsPropertyArray = SettingsProperty->AsArray();

	uint32 NumSettings;
	SettingsPropertyArray->GetNumElements(NumSettings);
	for (uint32 Index = 0; Index < NumSettings; ++Index)
	{
		TSharedPtr<IPropertyHandle> PlatformSettingsProperty = SettingsPropertyArray->GetElement(Index);
		if (PlatformSettingsProperty->IsValidHandle())
		{
			UObject* PlatformSettingsObject;
			FPropertyAccess::Result Result = PlatformSettingsProperty->GetValue(PlatformSettingsObject);
			if (Result == FPropertyAccess::Success)
			{
				if (UPlatformSettings* PlatformSettings = Cast<UPlatformSettings>(PlatformSettingsObject))
				{
					FAddPropertyParams Params = FAddPropertyParams()
						.UniqueId(PlatformSettings->GetFName())
						.AllowChildren(true)
						.CreateCategoryNodes(false);

					IDetailGroup& PlatformGroup = ChildBuilder.AddGroup(PlatformSettings->GetFName(), FText::FromName(PlatformSettings->GetPlatformIniName()));

					TSharedPtr<IPropertyHandle> PlatformSettingsPropertyArrayEntry = PlatformSettingsProperty->GetChildHandle(0);

					//FDetailWidgetRow& PlatformGroup = ChildBuilder.AddCustomRow(FText::FromString(PlatformSettings->GetPlatformIniName()));

					uint32 PlatformPropertiesChildHandleCount;
					PlatformSettingsPropertyArrayEntry->GetNumChildren(PlatformPropertiesChildHandleCount);
					for (uint32 PlatformPropertyIndex = 0; PlatformPropertyIndex < PlatformPropertiesChildHandleCount; PlatformPropertyIndex++)
					{
						TSharedPtr<IPropertyHandle> PlatformPropertyHandle = PlatformSettingsPropertyArrayEntry->GetChildHandle(PlatformPropertyIndex);
						PlatformGroup.AddPropertyRow(PlatformPropertyHandle.ToSharedRef());
					}
				}
			}
		}
	}
}