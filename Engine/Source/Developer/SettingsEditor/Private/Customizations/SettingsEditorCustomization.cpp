// Copyright Epic Games, Inc. All Rights Reserved.

#include "SettingsEditorCustomization.h"

#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "ISettingsEditorModule.h"
#include "ISettingsModule.h"
#include "ISettingsSection.h"
#include "HAL/FileManager.h"
#include "Misc/ConfigContext.h"
#include "Modules/ModuleManager.h"
#include "SettingsEditorLogs.h"
#include "UObject/Package.h"
#include "UObject/TextProperty.h"

#define LOCTEXT_NAMESPACE "SettingsEditorCustomization"

TMap<FString, TMap<FName, FString>> FSettingsEditorCustomization::ObjectDefaultPropertyValues;

namespace UE::SettingsEditor::Private
{
	static FAutoConsoleCommand ClearObjectDefaultPropertyValuesCache(
		TEXT("SettingsEditor.ClearObjectDefaultPropertyValuesCache"),
		TEXT("Clears the SettingsEditor objects default values cache to force it to rebuild."),
		FConsoleCommandDelegate::CreateStatic([]()
		{
			const FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
			const ISettingsModule& SettingsModule = FModuleManager::LoadModuleChecked<ISettingsModule>(TEXT("Settings"));
			TArray<FName> SettingsDetailsViewIds;
			SettingsModule.GetContainerNames(SettingsDetailsViewIds);
			// This will force cache to be rebuilt on next refresh which we will force below
			FSettingsEditorCustomization::ClearObjectDefaultPropertyValues();
			for (const FName& SettingsDetailsViewId : SettingsDetailsViewIds)
			{
				if (const TSharedPtr<IDetailsView> EditorDetailsView = PropertyEditorModule.FindDetailView(SettingsDetailsViewId))
				{
					EditorDetailsView->RequestForceRefresh();
				}
			}
		})
	);
}

TSharedRef<IDetailCustomization> FSettingsEditorCustomization::MakeInstance()
{
	return MakeShared<FSettingsEditorCustomization>();
}

void FSettingsEditorCustomization::ClearObjectDefaultPropertyValues()
{
	ObjectDefaultPropertyValues.Reset();
}

void FSettingsEditorCustomization::CustomizeDetails(IDetailLayoutBuilder& InDetailLayoutBuilder)
{
	TArray<TWeakObjectPtr<UObject>> SettingsObjects;
	InDetailLayoutBuilder.GetObjectsBeingCustomized(SettingsObjects);

	static const FTextFormat TokenCombineFormat = INVTEXT("{0} {1}");
	static const FText SearchKeyCVar = LOCTEXT("SearchKeyCVar", "CVar:");
	static const FTextFormat TokenDelimiter = INVTEXT("\"{0}\""); // Use for strict search instead of partial...

	ISettingsEditorModule::FSearchTerm* CVarSearchTerm = nullptr;
	if (ISettingsEditorModule* SettingsEditorModule = FModuleManager::GetModulePtr<ISettingsEditorModule>(TEXT("SettingsEditor")))
	{
		if (const TSharedPtr<IDetailsView> DetailsView = InDetailLayoutBuilder.GetDetailsViewSharedPtr())
		{
			const FName SearchContext = DetailsView->GetIdentifier();
			if (SettingsEditorModule->RegisterSearchTerm(SearchContext, SearchKeyCVar.ToString()))
			{
				CVarSearchTerm = SettingsEditorModule->FindSearchTerm(SearchContext, SearchKeyCVar.ToString());
				CVarSearchTerm->Label = LOCTEXT("SearchKeyCVarLabel", "CVar");
				CVarSearchTerm->Tooltip = LOCTEXT("SearchKeyCVarTooltip", "Filters properties to show those with a Console Variable");
				CVarSearchTerm->IconStyleSetName = "OutputLogStyle";
				CVarSearchTerm->IconStyleName = "DebugConsole.Icon";
				CVarSearchTerm->DisallowedSections.Add(TEXT("InputBindings")); // Keyboard Shortcuts
				CVarSearchTerm->AllowedSections.Add(NAME_None); // All section
				// Only show filter buttons in Project settings
				CVarSearchTerm->bShowFilter = SearchContext.IsEqual("Project");
			}
		}
	}

	for (const TWeakObjectPtr<UObject>& SettingsObject : SettingsObjects)
	{
		TStrongObjectPtr<UObject> PinnedSettingsObject = SettingsObject.Pin();
		if (!PinnedSettingsObject)
		{
			continue;
		}

		CacheResetDefaultPropertyValues(InDetailLayoutBuilder, PinnedSettingsObject.Get());

		const UClass* SettingsClass = SettingsObject->GetClass();
		for (const FProperty* SettingsProperty : TFieldRange<FProperty>(SettingsClass, EFieldIteratorFlags::IncludeSuper))
		{
			if (!SettingsProperty->HasAnyPropertyFlags(CPF_Config | CPF_GlobalConfig))
			{
				continue;
			}

			TSharedRef<IPropertyHandle> PropertyHandle = InDetailLayoutBuilder.GetProperty(SettingsProperty->GetFName(), SettingsClass);
			if (!PropertyHandle->IsValidHandle())
			{
				continue;
			}

			IDetailPropertyRow* PropertyRow = InDetailLayoutBuilder.EditDefaultProperty(PropertyHandle);
			if (!PropertyRow)
			{
				continue;
			}
			
			// Override reset to default
			PropertyRow->OverrideResetToDefault(
				FResetToDefaultOverride::Create(
					FIsResetToDefaultVisible::CreateSP(this, &FSettingsEditorCustomization::IsResetToDefaultVisible),
					FResetToDefaultHandler::CreateSP(this, &FSettingsEditorCustomization::OnResetToDefault)
				)
			);

			const FString& CVar = SettingsProperty->GetMetaData(TEXT("ConsoleVariable"));
			if (CVarSearchTerm != nullptr && !CVar.IsEmpty())
			{
				FDetailWidgetRow& Row = PropertyRow->CustomWidget(true);
				Row.NameContent()
				[
					PropertyHandle->CreatePropertyNameWidget()
				];

				Row.ValueContent()
				[
					PropertyHandle->CreatePropertyValueWidget()
				];

				// Allow searching of cvar in settings editor (for i.e. cvar="r.vsync")
				FText FilterText = FText::FormatOrdered(TokenDelimiter, FText::FromString(CVar));

				// Register search key and value for settings editor suggestions
				CVarSearchTerm->Values.Add(CVar);
				
				// Only allow this search term in the section specified by the settings object name
				{
					FString NativeDisplayName = PinnedSettingsObject->GetClass()->GetMetaData("DisplayName");
					if (NativeDisplayName.IsEmpty())
					{
						NativeDisplayName = FName::NameToDisplayString(PinnedSettingsObject->GetClass()->GetName(), false);
					}
					CVarSearchTerm->AllowedSections.Add(FName(NativeDisplayName));
				}

				FilterText = FText::FormatOrdered(TokenCombineFormat, SearchKeyCVar.ToLower(), FilterText);
				Row.FilterString(FilterText);
			}
		}
	}
}

void FSettingsEditorCustomization::OnResetToDefault(TSharedPtr<IPropertyHandle> InHandle)
{
	TArray<UObject*> SettingsOuterObjects;
	InHandle->GetOuterObjects(SettingsOuterObjects);
	if (SettingsOuterObjects.IsEmpty() || !SettingsOuterObjects[0])
	{
		return;
	}

	TMap<FName, FString>* PropertyValues = ObjectDefaultPropertyValues.Find(SettingsOuterObjects[0]->GetPathName());
	if (!PropertyValues)
	{
		return;
	}

	const FString* DefaultValuePtr = PropertyValues->Find(InHandle->GetProperty()->GetFName());
	if (!DefaultValuePtr)
	{
		return;
	}

	const FString DefaultValue = *DefaultValuePtr;
	InHandle->SetValueFromFormattedString(DefaultValue);
}

bool FSettingsEditorCustomization::IsResetToDefaultVisible(TSharedPtr<IPropertyHandle> InHandle)
{
	TArray<UObject*> SettingsOuterObjects;
	InHandle->GetOuterObjects(SettingsOuterObjects);
	if (SettingsOuterObjects.IsEmpty() || !SettingsOuterObjects[0])
	{
		return false;
	}

	TMap<FName, FString>* PropertyValues = ObjectDefaultPropertyValues.Find(SettingsOuterObjects[0]->GetPathName());
	if (!PropertyValues)
	{
		return false;
	}

	const FString* DefaultValuePtr = PropertyValues->Find(InHandle->GetProperty()->GetFName());
	if (!DefaultValuePtr)
	{
		return false;
	}

	FString CurrentValue;
	constexpr EPropertyPortFlags PortFlags = PPF_None;
	if (InHandle->GetValueAsFormattedString(CurrentValue, PortFlags) != FPropertyAccess::Success)
	{
		return false;
	}

	return !CurrentValue.Equals(*DefaultValuePtr, ESearchCase::CaseSensitive);
}

void FSettingsEditorCustomization::CacheResetDefaultPropertyValues(const IDetailLayoutBuilder& InDetailLayoutBuilder, UObject* InSettingsObject)
{
	if (!GConfig 
		|| !InSettingsObject 
		|| ObjectDefaultPropertyValues.Contains(InSettingsObject->GetPathName()))
	{
		// Done
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(FSettingsEditorCustomization::CacheResetDefaultPropertyValues);

	const UClass* SettingsClass = InSettingsObject->GetClass();
	const bool bIsDefaultConfig = SettingsClass->HasAnyClassFlags(CLASS_DefaultConfig);

	const FString ConfigFilePath = bIsDefaultConfig
		? SettingsClass->GetDefaultObject()->GetDefaultConfigFilename()
		: SettingsClass->GetConfigName();

	FString SectionName = SettingsClass->GetPathName();
	// Allows to override the config section name for backward compatibility
	InSettingsObject->OverrideConfigSection(SectionName);

	// Assume the current settings already match the defaults
	UObject* DefaultObject = InSettingsObject;
	
	// Cleanup temp object if any
	UObject* TempDefaultObject = nullptr;
	ON_SCOPE_EXIT
	{
		if (TempDefaultObject)
		{
			TempDefaultObject->MarkAsGarbage();
			TempDefaultObject = nullptr;
		}
	};

	// Skip creating temp object for abstract classes but still proceed further
	if (!SettingsClass->HasAnyClassFlags(CLASS_Abstract))
	{
		// If the section exists in the current config, then defaults have been overriden
		const FConfigFile* ConfigFile = GConfig->Find(ConfigFilePath);
		if (const FConfigSection* ConfigSection = ConfigFile ? ConfigFile->FindSection(SectionName) : nullptr)
		{
			// Allocate a new temporary object to access defaults
			TempDefaultObject = StaticAllocateObject(SettingsClass, GetTransientPackage(), NAME_None, RF_ClassDefaultObject | RF_Transient);

			EObjectInitializerOptions InitOptions = EObjectInitializerOptions::None;
			if (!SettingsClass->HasAnyClassFlags(CLASS_Native | CLASS_Intrinsic))
			{
				// Blueprint CDOs have their properties always initialized.
				InitOptions |= EObjectInitializerOptions::InitializeProperties;
			}

			// Initialize temporary object by loading the default config values
			(*SettingsClass->ClassConstructor)(FObjectInitializer(TempDefaultObject, nullptr, InitOptions));

			// Look into this temp object next
			DefaultObject = TempDefaultObject;
		}
	}

	// Get engine base config file eg: BaseInput.ini, BaseEngine.ini...
	const FString ConfigName = FPaths::GetBaseFilename(SettingsClass->GetConfigName());
	const FString BaseIniPath = FConfigCacheIni::NormalizeConfigIniPath((FPaths::EngineConfigDir() / FString::Printf(TEXT("Base%s.ini"), *ConfigName)));

	// Load base config from buffer but handle special symbol for arrays/sets...
	FConfigFile ConfigFile;
	if (FConfigFile* BaseConfigFile = GConfig->Find(BaseIniPath))
	{
		FString Buffer;
		BaseConfigFile->WriteToString(Buffer);
		ConfigFile.CombineFromBuffer(Buffer, FString(), /** HandleSymbolCommands */true);
	}

	// Add entry to avoid processing same object next time
	TMap<FName, FString>& PropertyValues = ObjectDefaultPropertyValues.FindOrAdd(InSettingsObject->GetPathName());
	for (FProperty* SettingsProperty : TFieldRange<FProperty>(SettingsClass, EFieldIteratorFlags::IncludeSuper))
	{
		if (!SettingsProperty->HasAnyPropertyFlags(CPF_Config | CPF_GlobalConfig))
		{
			continue;
		}

		TSharedRef<IPropertyHandle> PropertyHandle = InDetailLayoutBuilder.GetProperty(SettingsProperty->GetFName(), SettingsClass);
		if (!PropertyHandle->IsValidHandle())
		{
			continue;
		}
		
		FString DefaultValue;
		FString DefaultValueSource = TEXT("Class Default Object");
		bool bFound = false;

		// Look for cvar default value...
		const FString& ConsoleVar = PropertyHandle->GetMetaData(TEXT("ConsoleVariable"));
		if (!ConsoleVar.IsEmpty())
		{
			if (IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(*ConsoleVar))
			{
				DefaultValue = CVar->GetDefaultValue();

				const UEnum* Enum = nullptr;
				int64 EnumValue;
				LexFromString(EnumValue, *DefaultValue);
				
				// Sanitize Bool : "true" => "True", "false" => "False" since detection is case-sensitive
				if (const FBoolProperty* BoolProperty = CastField<FBoolProperty>(SettingsProperty))
				{
					DefaultValue = DefaultValue.ToBool() ? FCoreTexts::Get().True.ToString() : FCoreTexts::Get().False.ToString();
					bFound = true;
				}
				// Extract string value from enum : "0" becomes "FirstValueName"
				else if (const FByteProperty* ByteProperty = CastField<FByteProperty>(SettingsProperty))
				{
					Enum = ByteProperty->Enum.Get();
				}
				else if (const FEnumProperty* EnumProperty = CastField<FEnumProperty>(SettingsProperty))
				{
					Enum = EnumProperty->GetEnum();
				}
				// Property is a float/string/int or a type that does not need additional processing
				else
				{
					bFound = true;
				}

				if (Enum && Enum->IsValidEnumValue(EnumValue))
				{
					DefaultValue = Enum->GetNameStringByValue(EnumValue);
					bFound = true;
				}
				
				if (bFound)
				{
					DefaultValueSource = FString::Printf(TEXT("CVAR=%s"), *ConsoleVar);
				}
			}
		}

		// Look into base config file...
		if (!bFound)
		{
			const FString ConfigKey = SettingsProperty->GetName();
			const FConfigSection* ConfigSection = ConfigFile.FindSection(SectionName);
			if (ConfigSection && ConfigSection->Contains(*ConfigKey))
			{
				bFound = true;
				
				// Check if we need to retrieve a single or multiple values (array, set)
				if (SettingsProperty->IsA<FArrayProperty>() 
					|| SettingsProperty->IsA<FSetProperty>())
				{
					FProperty* InnerProp = nullptr;
					if (const FSetProperty* SetProperty = CastField<FSetProperty>(SettingsProperty))
					{
						InnerProp = SetProperty->ElementProp;
					}
					else if (const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(SettingsProperty))
					{
						InnerProp = ArrayProperty->Inner;
					}

					TArray<FString> DefaultValues;
					ConfigFile.GetArray(
						*SectionName,
						*ConfigKey,
						DefaultValues
					);

					// Wrap with "quotes" for string otherwise format is not valid to compare import vs export, eg: ("/Game","/Engine") vs (/Game, /Engine)
					if (InnerProp && (InnerProp->IsA<FStrProperty>() 
						|| InnerProp->IsA<FNameProperty>() 
						|| InnerProp->IsA<FTextProperty>()))
					{
						for (FString& Value : DefaultValues)
						{
							Value = FString::Printf(TEXT("\"%s\""), *Value);
						}
					}

					// Wrap in array (...) otherwise format is not valid to compare import vs export
					DefaultValue = FString::Printf(TEXT("(%s)"), *FString::Join(DefaultValues, TEXT(",")));
				}
				else
				{
					// Maps are serialized in a single line eg: MapPropertyName=((Key, "Value"),(Key2, "Value2"))
					ConfigFile.GetString(
						*SectionName,
						*ConfigKey,
						DefaultValue
					);
				}

				// We need to normalize values to avoid precision and format issues 
				// eg: Current Value (R=0.594000,G=0.019700,B=0.000000,A=1.000000) VS Config Value (R=0.594,G=0.0197,B=0.0,A=1.0) are not equal
				FString Normalized;

				// Allocate a buffer that satisfies the property size and alignment
				const int32 PropSize = SettingsProperty->GetSize();
				const int32 PropAlignment = SettingsProperty->GetMinAlignment();
				if (void* TempBuffer = FMemory::Malloc(PropSize, PropAlignment))
				{
					SettingsProperty->InitializeValue(TempBuffer);

					if (const TCHAR* Result = SettingsProperty->ImportText_Direct(*DefaultValue, TempBuffer, nullptr, PPF_None))
					{
						// Export back through the same path so both strings are formatted identically...
						SettingsProperty->ExportTextItem_Direct(Normalized, TempBuffer, nullptr, nullptr, PPF_None);
					}
					else
					{
						Normalized = DefaultValue;
					}

					SettingsProperty->DestroyValue(TempBuffer);
					FMemory::Free(TempBuffer);
				}
				else
				{
					Normalized = DefaultValue;
				}

				DefaultValue = Normalized;
				DefaultValueSource = FString::Printf(TEXT("INI=%s[%s][%s]"), *BaseIniPath, *SectionName, *SettingsProperty->GetName());	
			}
		}

		// Look into the default object defined previously...
		if (!bFound && !GetValueFromObject(PropertyHandle, DefaultObject, DefaultValue))
		{
			UE_LOGF(LogSettingsEditor, Warning, "Could not retrieve default value for property %ls in %ls", *PropertyHandle->GetPropertyDisplayName().ToString(), *DefaultObject->GetName())
			continue;
		}

		PropertyValues.Add(SettingsProperty->GetFName(), DefaultValue);

		// Set default value and default value source in tooltip for convenience
		const FText PropertyTooltip = FText::Format(
			INVTEXT("{0}\n\nDefault Value: {1}\n\nDefault Value Source: {2}"), 
			PropertyHandle->GetToolTipText(), 
			FText::FromString(*DefaultValue),
			FText::FromString(*DefaultValueSource)
		);

		PropertyHandle->SetToolTipText(PropertyTooltip);
	}
}

bool FSettingsEditorCustomization::GetValueFromObject(const TSharedPtr<IPropertyHandle>& InPropertyHandle, UObject* InValueSource, FString& OutValue)
{
	if (!InValueSource || !InPropertyHandle || !InPropertyHandle->GetProperty())
	{
		return false;
	}

	const FProperty* SettingsProperty = InPropertyHandle->GetProperty();
	const void* PropertyAddress = InPropertyHandle->GetValueBaseAddress(reinterpret_cast<uint8*>(InValueSource));

	constexpr uint32 PortFlags = PPF_None;
	if (InPropertyHandle->GetArrayIndex() == INDEX_NONE && SettingsProperty->ArrayDim > 1)
	{
		FArrayProperty::ExportTextInnerItem(OutValue, SettingsProperty, PropertyAddress, SettingsProperty->ArrayDim,
			PropertyAddress, SettingsProperty->ArrayDim, nullptr, PortFlags);
	}
	else
	{
		SettingsProperty->ExportTextItem_Direct(OutValue, PropertyAddress, PropertyAddress, nullptr, PortFlags);
	}

	const UEnum* Enum = nullptr;
	int64 EnumValue = 0;
	if (const FByteProperty* ByteProperty = CastField<FByteProperty>(SettingsProperty))
	{
		if (ByteProperty->Enum != nullptr)
		{
			Enum = ByteProperty->Enum.Get();
			EnumValue = ByteProperty->GetPropertyValue(PropertyAddress);
		}
	}
	else if (const FEnumProperty* EnumProperty = CastField<FEnumProperty>(SettingsProperty))
	{
		Enum = EnumProperty->GetEnum();
		EnumValue = EnumProperty->GetUnderlyingProperty()->GetSignedIntPropertyValue(PropertyAddress);
	}

	if (Enum)
	{
		if (Enum->IsValidEnumValue(EnumValue))
		{
			OutValue = Enum->GetNameStringByValue(EnumValue);
		}
		else
		{
			return false;
		}
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
