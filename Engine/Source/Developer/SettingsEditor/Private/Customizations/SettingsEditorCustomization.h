// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "PropertyHandle.h"

//////////////////////////////////////////////////////////////////////////
// FSettingsEditorCustomization

struct FInstancedPropertyBag;

class FSettingsEditorCustomization : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance();
	static void ClearObjectDefaultPropertyValues();

	//~ Begin IDetailCustomization
	virtual void CustomizeDetails(IDetailLayoutBuilder& InDetailLayoutBuilder) override;
	//~ End IDetailCustomization

private:
	/** Stores all settings object default property values ObjectPath->PropertyName->Value */
	static TMap<FString, TMap<FName, FString>> ObjectDefaultPropertyValues;
	static bool GetValueFromObject(const TSharedPtr<IPropertyHandle>& InPropertyHandle, UObject* InValueSource, FString& OutValue);

	/** Saves once at startup all settings object default property values */
	void CacheResetDefaultPropertyValues(const IDetailLayoutBuilder& InDetailLayoutBuilder, UObject* InSettingsObject);
	void OnResetToDefault(TSharedPtr<IPropertyHandle> InHandle);
	bool IsResetToDefaultVisible(TSharedPtr<IPropertyHandle> InHandle);
};
