// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "UObject/Object.h"
#include "SettingsEditorMenuContext.generated.h"

class SSettingsEditor;

/** Context passed in UToolMenu when generating menu entries for settings editor search toolbar */
UCLASS()
class USettingsEditorMenuContext : public UObject
{
	GENERATED_BODY()

public:
	TSharedPtr<SSettingsEditor> GetSettingsEditor() const
	{
		return SettingsEditorWeak.Pin();
	}

	TWeakPtr<SSettingsEditor> SettingsEditorWeak;
};