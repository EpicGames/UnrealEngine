// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/WeakObjectPtr.h"
#include "ISettingsContainer.h"
#include "ISettingsEditorModel.h"
#include "ISettingsCategory.h"
#include "ISettingsSection.h"

/**
 * Implements a view model for the settings editor widget.
 */
class FSettingsEditorModel
	: public ISettingsEditorModel
{
public:

	/**
	 * Creates and initializes a new instance.
	 *
	 * @param InSettingsContainer The settings container to use.
	 */
	FSettingsEditorModel( const ISettingsContainerRef& InSettingsContainer )
		: SettingsContainer(InSettingsContainer)
	{
		SettingsContainer->OnSectionRemoved().AddRaw(this, &FSettingsEditorModel::HandleSettingsContainerSectionRemoved);
	}

	/** Virtual destructor. */
	virtual ~FSettingsEditorModel()
	{
		SettingsContainer->OnSectionRemoved().RemoveAll(this);
	}

public:

	// ISettingsEditorModel interface

	virtual const ISettingsSectionPtr& GetSelectedSection() const override
	{
		return SelectedSection;
	}

	virtual const ISettingsContainerRef& GetSettingsContainer() const override
	{
		return SettingsContainer;
	}

	virtual FSimpleMulticastDelegate& OnSelectionChanged() override
	{
		return OnSelectionChangedDelegate;
	}

	virtual void SelectSection( const ISettingsSectionPtr& Section ) override
	{
		if (Section == SelectedSection)
		{
			return;
		}

		SelectedSection = Section;
		OnSelectionChangedDelegate.Broadcast();
	}

	virtual ISettingsSectionPtr GetSectionFromSectionObject(const UObject* SectionObject) const override
	{
		TArray<ISettingsCategoryPtr> Categories;
		GetSettingsContainer()->GetCategories(Categories);

		for(ISettingsCategoryPtr& Category : Categories)
		{
			TArray<ISettingsSectionPtr> Sections;
			Category->GetSections(Sections);

			for(ISettingsSectionPtr& Section : Sections)
			{
				if(Section->GetSettingsObject() == SectionObject)
				{
					return Section;
				}
			}
		}

		return nullptr;
	}

private:

	/** Handles the removal of sections from the settings container. */
	void HandleSettingsContainerSectionRemoved( const ISettingsSectionRef& Section )
	{
		if (SelectedSection == Section)
		{
			SelectSection(nullptr);
		}
	}

private:

	/** Holds the currently selected settings section. */
	ISettingsSectionPtr SelectedSection;

	/** Holds a reference to the settings container. */
	ISettingsContainerRef SettingsContainer;

private:

	/** Holds a delegate that is executed when the selected settings section has changed. */
	FSimpleMulticastDelegate OnSelectionChangedDelegate;
};
