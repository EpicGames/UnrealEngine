// Copyright Epic Games, Inc. All Rights Reserved.

#include "Customizations/PerPlatformSettingsCustomization.h"
#include "DetailRowMenuContext.h"
#include "Engine/DeveloperSettings.h"
#include "Engine/PlatformSettings.h"
#include "Framework/Notifications/NotificationManager.h"
#include "IDetailTreeNode.h"
#include "ISettingsContainer.h"
#include "ISettingsEditorModel.h"
#include "ISettingsEditorModule.h"
#include "ISettingsModule.h"
#include "Models/SettingsEditorModel.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorClipboard.h"
#include "PropertyEditorDelegates.h"
#include "PropertyEditorModule.h"
#include "SettingsEditorLogs.h"
#include "ToolMenus.h"
#include "UObject/UObjectHash.h"
#include "UObject/UObjectIterator.h"
#include "UObject/UObjectThreadContext.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Widgets/SSettingsEditor.h"

DEFINE_LOG_CATEGORY(LogSettingsEditor);

#define LOCTEXT_NAMESPACE "SSettingsEditor"

/** Holds auto discovered settings information so that they can be unloaded automatically when refreshing. */
struct FRegisteredSettings
{
	FName ContainerName;
	FName CategoryName;
	FName SectionName;
};

/** Manages the notification for when the application needs to be restarted due to a settings change */
class FApplicationRestartRequiredNotification
{
public:
	void SetOnRestartApplicationCallback( FSimpleDelegate InRestartApplicationDelegate )
	{
		RestartApplicationDelegate = InRestartApplicationDelegate;
	}

	void OnRestartRequired()
	{
		TSharedPtr<SNotificationItem> NotificationPin = NotificationPtr.Pin();
		if (NotificationPin.IsValid() || !RestartApplicationDelegate.IsBound())
		{
			return;
		}
		
		FNotificationInfo Info( LOCTEXT("RestartRequiredTitle", "Restart required to apply new settings") );

		// Add the buttons with text, tooltip and callback
		Info.ButtonDetails.Add(FNotificationButtonInfo(
			LOCTEXT("RestartNow", "Restart Now"), 
			LOCTEXT("RestartNowToolTip", "Restart now to finish applying your new settings."), 
			FSimpleDelegate::CreateRaw(this, &FApplicationRestartRequiredNotification::OnRestartClicked))
			);
		Info.ButtonDetails.Add(FNotificationButtonInfo(
			LOCTEXT("RestartLater", "Restart Later"), 
			LOCTEXT("RestartLaterToolTip", "Dismiss this notificaton without restarting. Some new settings will not be applied."), 
			FSimpleDelegate::CreateRaw(this, &FApplicationRestartRequiredNotification::OnDismissClicked))
			);

		// We will be keeping track of this ourselves
		Info.bFireAndForget = false;

		// Set the width so that the notification doesn't resize as its text changes
		Info.WidthOverride = 300.0f;

		Info.bUseLargeFont = false;
		Info.bUseThrobber = false;
		Info.bUseSuccessFailIcons = false;

		// Launch notification
		NotificationPtr = FSlateNotificationManager::Get().AddNotification(Info);
		NotificationPin = NotificationPtr.Pin();

		if (NotificationPin.IsValid())
		{
			NotificationPin->SetCompletionState(SNotificationItem::CS_Pending);
		}
	}

private:
	void OnRestartClicked()
	{
		TSharedPtr<SNotificationItem> NotificationPin = NotificationPtr.Pin();
		if (NotificationPin.IsValid())
		{
			NotificationPin->SetText(LOCTEXT("RestartingNow", "Restarting..."));
			NotificationPin->SetCompletionState(SNotificationItem::CS_Success);
			NotificationPin->ExpireAndFadeout();
			NotificationPtr.Reset();
		}

		RestartApplicationDelegate.ExecuteIfBound();
	}

	void OnDismissClicked()
	{
		TSharedPtr<SNotificationItem> NotificationPin = NotificationPtr.Pin();
		if (NotificationPin.IsValid())
		{
			NotificationPin->SetText(LOCTEXT("RestartDismissed", "Restart Dismissed..."));
			NotificationPin->SetCompletionState(SNotificationItem::CS_None);
			NotificationPin->ExpireAndFadeout();
			NotificationPtr.Reset();
		}
	}

	/** Used to reference to the active restart notification */
	TWeakPtr<SNotificationItem> NotificationPtr;

	/** Used to actually restart the application */
	FSimpleDelegate RestartApplicationDelegate;
};


/**
 * Implements the SettingsEditor module.
 */
class FSettingsEditorModule
	: public ISettingsEditorModule
{
public:

	FSettingsEditorModule()
		: bAreSettingsStale(true)
	{
	}

	// IModuleInterface interface

	virtual void StartupModule() override
	{
		FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
		PropertyModule.RegisterCustomPropertyTypeLayout(
			StaticStruct<FPerPlatformSettings>()->GetFName(),
			FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FPerPlatformSettingsCustomization::MakeInstance)
		);

		FModuleManager::Get().OnModulesChanged().AddRaw(this, &FSettingsEditorModule::ModulesChangesCallback);
		
		RegisterSettingsActions();
	}

	virtual void ShutdownModule() override
	{
		ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");

		if ( SettingsModule != nullptr )
		{
			UnregisterAutoDiscoveredSettings(*SettingsModule);
		}

		FModuleManager::Get().OnModulesChanged().RemoveAll(this);
		
		UnregisterSettingsActions();
	}

	// ISettingsEditorModule interface

	virtual TSharedRef<SWidget> CreateEditor( const TSharedRef<ISettingsEditorModel>& Model ) override
	{
		UpdateSettings(true);

		TSharedRef<SWidget> Editor = SNew(SSettingsEditor, Model)
			.OnApplicationRestartRequired(FSimpleDelegate::CreateRaw(this, &FSettingsEditorModule::OnApplicationRestartRequired));

		ClearStaleEditorWidgets();
		EditorWidgets.Add(Editor);

		return Editor;
	}

	virtual ISettingsEditorModelRef CreateModel( const TSharedRef<ISettingsContainer>& SettingsContainer ) override
	{
		return MakeShareable(new FSettingsEditorModel(SettingsContainer));
	}

	virtual void OnApplicationRestartRequired() override
	{
		ApplicationRestartRequiredNotification.OnRestartRequired();
	}

	virtual void SetRestartApplicationCallback( FSimpleDelegate InRestartApplicationDelegate ) override
	{
		ApplicationRestartRequiredNotification.SetOnRestartApplicationCallback(InRestartApplicationDelegate);
	}

	virtual void SetShouldRegisterSettingCallback(FShouldRegisterSettingsDelegate InShouldRegisterSettingDelegate) override
	{
		ShouldRegisterSettingsDelegate = MoveTemp(InShouldRegisterSettingDelegate);
	}

	virtual bool RegisterSearchTerm(FName InContainerName, const FString& InKey) override
	{
		if (InContainerName.IsNone() || InKey.IsEmpty())
		{
			return false;
		}

		// Already registered ?
		if (FSearchTerm* SearchTerm = FindSearchTerm(InContainerName, InKey))
		{
			return true;
		}

		TMap<FString, FSearchTerm>& ContainerKeys = SearchTerms.FindOrAdd(InContainerName);
		ContainerKeys.FindOrAdd(InKey);

		// Resort to display in alphabetical order
		ContainerKeys.KeySort(TLess<FString>());

		// Notify
		OnSearchTermsChangedDelegate.Broadcast(InContainerName, InKey);

		return true;
	}
	
	virtual FOnSearchTermsChanged::RegistrationType& OnSearchTermsChanged() override
	{
		return OnSearchTermsChangedDelegate;
	}

	virtual FSearchTerm* FindSearchTerm(FName InContainerName, const FString& InKey) override
	{
		if (TMap<FString, FSearchTerm>* ContainerSearchTerms = SearchTerms.Find(InContainerName))
		{
			return ContainerSearchTerms->Find(InKey);
		}

		return nullptr;
	}
	
	virtual const FSearchTerm* FindSearchTerm(FName InContainerName, const FString& InKey) const override
	{
		if (const TMap<FString, FSearchTerm>* ContainerSearchTerms = SearchTerms.Find(InContainerName))
		{
			return ContainerSearchTerms->Find(InKey);
		}

		return nullptr;
	}

	virtual const TMap<FString, FSearchTerm>* FindSearchTerms(FName InContainerName) const override
	{
		return SearchTerms.Find(InContainerName);
	}

private:

	void ModulesChangesCallback(FName ModuleName, EModuleChangeReason ReasonForChange)
	{
		ClearStaleEditorWidgets();
		bAreSettingsStale = true;
		UpdateSettingsNowOrDeferIfUnsafe();
	}

	virtual void UpdateSettings(bool bForce = false) override
	{
		if ( ( AnyActiveSettingsEditor() || bForce ) && bAreSettingsStale )
		{
			bAreSettingsStale = false;

			ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");

			if ( SettingsModule != nullptr )
			{
				UnregisterAutoDiscoveredSettings(*SettingsModule);
				RegisterAutoDiscoveredSettings(*SettingsModule);
			}
		}
	}

	void ClearStaleEditorWidgets()
	{
		for ( int32 i = 0; i < EditorWidgets.Num(); i++ )
		{
			if ( !EditorWidgets[i].IsValid() )
			{
				EditorWidgets.RemoveAtSwap(i);
				i--;
			}
		}
	}

	void UpdateSettingsNowOrDeferIfUnsafe()
	{
		// Updating can cause UObject::ProcessEvent to be called, e.g. via FDetailPropertyRow::IsEditConditionMet.
		// UObject::ProcessEvent is not allowed to be called during PostLoad.
		// We can get here by having Editor Settings open and a UObject's PostLoad loading a module during PostLoad, which triggers
		// ModulesChangesCallback and in turns calls UpdateSettings.
		const bool bIsSafeToUpdateNow = !FUObjectThreadContext::Get().IsRoutingPostLoad;
		if (bIsSafeToUpdateNow)
		{
			UpdateSettings();
		}
		else
		{
			DeferUpdateSettingsToNextTick();
		}
	}
	
	void DeferUpdateSettingsToNextTick()
	{
		if (bHasDeferredUpdate)
		{
			return;
		}
		
		bHasDeferredUpdate = true;
		ExecuteOnGameThread(TEXT("UpdateSettings"), [this]
		{
			bHasDeferredUpdate = false;
			UpdateSettingsNowOrDeferIfUnsafe();
		});
	}
	
	bool AnyActiveSettingsEditor()
	{
		return EditorWidgets.Num() > 0;
	}

private:

	void RegisterAutoDiscoveredSettings(ISettingsModule& SettingsModule)
	{
		// Find game object
		for ( TObjectIterator<UDeveloperSettings> SettingsIt(RF_NoFlags); SettingsIt; ++SettingsIt )
		{
			if ( UDeveloperSettings* Settings = *SettingsIt )
			{
				// Only Add the CDO of any UDeveloperSettings objects.
				if ( Settings->HasAnyFlags(RF_ClassDefaultObject) && !Settings->GetClass()->HasAnyClassFlags(CLASS_Deprecated | CLASS_Abstract) )
				{
					// Ignore the setting if it's specifically the UDeveloperSettings or other abstract settings classes
					if ( Settings->GetClass()->HasAnyClassFlags(CLASS_Abstract) || !Settings->SupportsAutoRegistration() )
					{
						continue;
					}

					// Let an external module decide if the setting should be registered.
					if (ShouldRegisterSettingsDelegate.IsBound() && !ShouldRegisterSettingsDelegate.Execute(Settings))
					{
						continue;
					}

					RegisterDeveloperSettings(SettingsModule, Settings);
				}
			}
		}
	}

	void RegisterDeveloperSettings(ISettingsModule& SettingsModule, UDeveloperSettings* Settings)
	{
		FRegisteredSettings Registered;
		Registered.ContainerName = Settings->GetContainerName();
		Registered.CategoryName = Settings->GetCategoryName();
		Registered.SectionName = Settings->GetSectionName();

		TSharedPtr<SWidget> CustomWidget = Settings->GetCustomSettingsWidget();
		if (CustomWidget.IsValid())
		{
			// Add Settings
			SettingsModule.RegisterSettings(Registered.ContainerName, Registered.CategoryName, Registered.SectionName,
				Settings->GetSectionText(),
				Settings->GetSectionDescription(),
				CustomWidget.ToSharedRef()
			);
		}
		else
		{
			// Add Settings
			SettingsModule.RegisterSettings(Registered.ContainerName, Registered.CategoryName, Registered.SectionName,
				Settings->GetSectionText(),
				Settings->GetSectionDescription(),
				Settings
			);
		}

		AutoDiscoveredSettings.Add(Registered);
	}

	void UnregisterAutoDiscoveredSettings(ISettingsModule& SettingsModule)
	{
		// Unregister any auto discovers settings.
		for ( const FRegisteredSettings& Settings : AutoDiscoveredSettings )
		{
			SettingsModule.UnregisterSettings(Settings.ContainerName, Settings.CategoryName, Settings.SectionName);
		}

		AutoDiscoveredSettings.Reset();
	}

private:
	void RegisterSettingsActions()
	{
		if (FPropertyEditorModule* PropertyModule = FModuleManager::GetModulePtr<FPropertyEditorModule>("PropertyEditor"))
		{
			PropertyModule->GetGlobalRowExtensionDelegate().AddStatic(&FSettingsEditorModule::OnExtendRowExtensionButtons);
		}
	}
	
	void UnregisterSettingsActions() const
	{
		if (FPropertyEditorModule* PropertyModule = FModuleManager::GetModulePtr<FPropertyEditorModule>("PropertyEditor"))
		{
			PropertyModule->GetGlobalRowExtensionDelegate().RemoveAll(this);
		}
	}

	static void ExtendRowContextMenu()
	{
		static bool bMenuExtended = false;

		if (!bMenuExtended)
		{
			UToolMenus* Menus = UToolMenus::Get();

			checkf(Menus, TEXT("ToolMenus not available"));

			if (UToolMenu* ContextMenu = Menus->FindMenu(UE::PropertyEditor::RowContextMenuName))
			{
				ContextMenu->AddDynamicSection(
					TEXT("FillSettingsRowContextSection"), 
					FNewToolMenuDelegate::CreateStatic(&FSettingsEditorModule::FillSettingsRowContextSection)
				);
				
				bMenuExtended =  true;
			}
		}
	}

	static void FillSettingsRowContextSection(UToolMenu* InToolMenu)
	{
		if (!InToolMenu)
		{
			return;
		}

		UDetailRowMenuContext* RowContext = InToolMenu->FindContext<UDetailRowMenuContext>();
		if (!RowContext 
			|| RowContext->PropertyHandles.IsEmpty() 
			|| !RowContext->PropertyHandles[0].IsValid())
		{
			return;
		}

		// Only fill menu with a property linked to a CVar
		const FString CVar = GetCVarFromPropertyHandle(RowContext->PropertyHandles[0]);
		if (CVar.IsEmpty())
		{
			return;
		}

		InToolMenu->AddMenuEntry(
			TEXT("CopyCVarName"),
			FToolMenuEntry::InitMenuEntry(
				TEXT("CopyCvarName"),
				LOCTEXT("CopyCVarName", "Copy CVar Name"),
				FText::Format(LOCTEXT("CopyPropertyCVarNameToolTip", "Copy the console variable name of this property to the system clipboard:\n{0}"), FText::FromString(CVar)),
				FSlateIcon(FCoreStyle::Get().GetStyleSetName(), "GenericCommands.Copy"),
				FUIAction(
					FExecuteAction::CreateLambda(&FSettingsEditorModule::CopyCVarToClipboard, CVar)
				)
			)
		);
	}

	static void OnExtendRowExtensionButtons(const FOnGenerateGlobalRowExtensionArgs& InArgs, TArray<FPropertyRowExtensionButton>& OutExtensionButtons)
	{
		if (!InArgs.PropertyHandle.IsValid() || !InArgs.PropertyHandle->IsValidHandle())
		{
			return;
		}

		const TSharedPtr<IDetailTreeNode> DetailNode = InArgs.OwnerTreeNode.Pin();
		if (!DetailNode.IsValid())
		{
			return;
		}

		const TSharedPtr<IDetailsView> DetailsView = DetailNode->GetNodeDetailsViewSharedPtr();
		if (!DetailsView)
		{
			return;
		}
		
		// Only extend rows that belong to settings editor
		const FName ViewIdentifier = DetailsView->GetIdentifier();
		if (ViewIdentifier != TEXT("Project") && ViewIdentifier != TEXT("Editor"))
		{
			return;
		}

		// Only extend rows with a property linked to a CVar
		const FString CVarMetaData = GetCVarFromPropertyHandle(InArgs.PropertyHandle);
		if (CVarMetaData.IsEmpty())
		{
			return;
		}
		
		// Extend context menu here to ensure menu is registered and ready, this is only done once
		ExtendRowContextMenu();

		FPropertyRowExtensionButton& ExtensionButton = OutExtensionButtons.Add_GetRef(FPropertyRowExtensionButton());
		ExtensionButton.Icon = FSlateIcon("OutputLogStyle", "DebugConsole.Icon");
		ExtensionButton.Label = LOCTEXT("ExtensionCopyCVarName", "Copy CVar Name");
		ExtensionButton.ToolTip = FText::Format(
			LOCTEXT("ExtensionCopyCVarNameTooltip", "Copy CVar linked to this property row in the clipboard:\n{0}"),
			FText::FromString(CVarMetaData)
		);
		ExtensionButton.UIAction = FUIAction(
			FExecuteAction::CreateLambda(&FSettingsEditorModule::CopyCVarToClipboard, CVarMetaData)
		);
	}
	
	static FString GetCVarFromPropertyHandle(const TSharedPtr<IPropertyHandle>& InPropertyHandle)
	{
		if (InPropertyHandle)
		{
			const FString ConsoleVariableMeta = TEXT("ConsoleVariable");
			if (InPropertyHandle->HasMetaData(*ConsoleVariableMeta))
			{
				return InPropertyHandle->GetMetaData(*ConsoleVariableMeta);
			}
		}

		return FString();
	}
	
	static void CopyCVarToClipboard(const FString& InCVar)
	{
		FPropertyEditorClipboard::ClipboardCopy(*InCVar);

		// Notify of the action
		FNotificationInfo NotificationInfo(LOCTEXT("CopyCVarNotification", "CVar copied to clipboard"));
		NotificationInfo.ExpireDuration = 1.f;
		NotificationInfo.bFireAndForget = true;
		NotificationInfo.Image = FAppStyle::GetBrush("Icons.SuccessWithColor");
		FSlateNotificationManager::Get().AddNotification(NotificationInfo);
	}

	FApplicationRestartRequiredNotification ApplicationRestartRequiredNotification;

	/** The list of auto discovered settings that need to be unregistered. */
	TArray<FRegisteredSettings> AutoDiscoveredSettings;

	/** Living editor widgets that have been handed out. */
	TArray< TWeakPtr<SWidget> > EditorWidgets;

	/** Flag if the settings are stale currently and need to be refreshed. */
	bool bAreSettingsStale;

	/** Flag to see whether the settings should be updated next tick. */
	bool bHasDeferredUpdate = false;

	/** Delegate called to decide if a settings object should be registered with the module. */
	FShouldRegisterSettingsDelegate ShouldRegisterSettingsDelegate;
	
	/** Used to store Context->Key->Values for search suggestions */
	TMap<FName, TMap<FString, FSearchTerm>> SearchTerms;

	/** Used to broadcast change events */
	FOnSearchTermsChanged OnSearchTermsChangedDelegate;
};


IMPLEMENT_MODULE(FSettingsEditorModule, SettingsEditor);


#undef LOCTEXT_NAMESPACE
