// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SSettingsEditor.h"

#include "Editor/EditorWidgets/Public/SAssetSearchBox.h"
#include "UObject/UnrealType.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"
#include "Widgets/SBoxPanel.h"
#include "Layout/WidgetPath.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Styling/AppStyle.h"
#include "AnalyticsEventAttribute.h"
#include "DesktopPlatformModule.h"
#include "EngineAnalytics.h"
#include "PropertyEditorModule.h"
#include "IDetailsView.h"
#include "SSettingsEditorCategoryTree.h"
#include "Widgets/Input/SHyperlink.h"
#include "Widgets/SSettingsSectionHeader.h"
#include "SSettingsEditorCheckoutNotice.h"
#include "ToolMenuEntry.h"
#include "ToolMenus.h"
#include "ToolMenu.h"
#include "HAL/PlatformFileManager.h"
#include "HAL/PlatformFile.h"
#include "Containers/Ticker.h"
#include "Customizations/SettingsEditorCustomization.h"
#include "Framework/Notifications/NotificationManager.h"
#include "HAL/FileManager.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/MessageDialog.h"
#include "Misc/TransactionObjectEvent.h"
#include "Models/SettingsEditorMenuContext.h"
#include "Sidebar/ISidebarDrawerContent.h"
#include "Sidebar/SSidebarContainer.h"
#include "Widgets/Layout/SBox.h"

#define LOCTEXT_NAMESPACE "SSettingsEditor"

FLazyName SSettingsEditor::SettingsEditorToolbarName = TEXT("SettingsEditor.SearchToolbar");
FLazyName SSettingsEditor::SettingsEditorSidebarId = TEXT("CategoryTree");
const FTextFormat SSettingsEditor::TokenCombineFormat = INVTEXT("{0} {1}");
const FTextFormat SSettingsEditor::TokenDelimiter = INVTEXT("\"{0}\"");

/** For the category tree sidebar */
class FSettingsEditorDrawerContent : public ISidebarDrawerContent
{
public:
	explicit FSettingsEditorDrawerContent(TSharedRef<SWidget> InSidebarContent)
		: SidebarContent(InSidebarContent)
	{}

	//~ Begin ISidebarDrawerContent
	virtual FName GetUniqueId() const override { return SSettingsEditor::SettingsEditorSidebarId; }
	virtual FName GetSectionId() const override { return TEXT("Settings"); }
	virtual FText GetSectionDisplayText() const override { return LOCTEXT("SettingsSection", "Settings"); }
	virtual TSharedRef<SWidget> CreateContentWidget() override { return SidebarContent; }
	//~ End ISidebarDrawerContent

private:
	TSharedRef<SWidget> SidebarContent;
};

/* SSettingsEditor structors
 *****************************************************************************/

SSettingsEditor::~SSettingsEditor()
{
	FCoreUObjectDelegates::OnObjectTransacted.RemoveAll(this);
	Model->OnSelectionChanged().RemoveAll(this);
	if (ISettingsEditorModule* SettingsEditorModule = FModuleManager::GetModulePtr<ISettingsEditorModule>(TEXT("SettingsEditor")))
	{
		SettingsEditorModule->OnSearchTermsChanged().RemoveAll(this);
	}
	if (MenuToolbarUpdateHandle.IsValid())
	{
		FTSTicker::GetCoreTicker().RemoveTicker(MenuToolbarUpdateHandle);
		MenuToolbarUpdateHandle.Reset();
	}
}


/* SSettingsEditor interface
 *****************************************************************************/

void SSettingsEditor::Construct( const FArguments& InArgs, const ISettingsEditorModelRef& InModel )
{
	Model = InModel;
	SettingsContainer = InModel->GetSettingsContainer();
	OnApplicationRestartRequiredDelegate = InArgs._OnApplicationRestartRequired;

	// initialize settings view
	FDetailsViewArgs DetailsViewArgs;
	{
		DetailsViewArgs.bAllowSearch = true;
		DetailsViewArgs.bHideSelectionTip = true;
		DetailsViewArgs.bLockable = false;
		DetailsViewArgs.bSearchInitialKeyFocus = true;
		DetailsViewArgs.bUpdatesFromSelection = false;
		DetailsViewArgs.NotifyHook = this;
		DetailsViewArgs.bShowOptions = true;
		DetailsViewArgs.bShowModifiedPropertiesOption = false;
		DetailsViewArgs.bShowAnimatedPropertiesOption = false;
		DetailsViewArgs.bShowDifferingPropertiesOption = false;
		DetailsViewArgs.bShowKeyablePropertiesOption = false;
		DetailsViewArgs.bShowHiddenPropertiesWhilePlayingOption = false;
		DetailsViewArgs.bShowPropertyMatrixButton = false;
		DetailsViewArgs.bShowSectionSelector = true;
		DetailsViewArgs.bAllowMultipleTopLevelObjects = true;
		DetailsViewArgs.bCustomNameAreaLocation = true;
		DetailsViewArgs.bCustomFilterAreaLocation = true;
		DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
		DetailsViewArgs.RightColumnMinWidth = 50.f;
		// Used to identify details view
		DetailsViewArgs.ViewIdentifier = SettingsContainer ? SettingsContainer->GetName() : TEXT("SSettingsEditor");
	}

	SettingsView = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor").CreateDetailView(DetailsViewArgs);
	SettingsView->SetVisibility(TAttribute<EVisibility>::CreateSP(this, &SSettingsEditor::HandleSettingsViewVisibility));
	SettingsView->SetIsPropertyEditingEnabledDelegate(FIsPropertyEditingEnabled::CreateSP(this, &SSettingsEditor::HandleSettingsViewEnabled));
	SettingsView->SetGenericLayoutDetailsDelegate(FOnGetDetailCustomizationInstance::CreateStatic(&FSettingsEditorCustomization::MakeInstance));

	TSharedPtr<FSettingsDetailRootObjectCustomization> RootObjectCustomization = MakeShareable(new FSettingsDetailRootObjectCustomization(Model, SettingsView.ToSharedRef()));
	RootObjectCustomization->Initialize();
	SettingsView->SetRootObjectCustomizationInstance(RootObjectCustomization);
	
	// Extract specific widgets only, we don't need the search bar since we are creating our own...
	TSharedPtr<SWidget> DetailsViewOptions;
	TSharedPtr<SWidget> DetailsViewSections;
	if (const TSharedPtr<SWidget> FilterWidget = SettingsView->GetFilterAreaWidget())
	{
		TFunction<TSharedPtr<SWidget>(TSharedRef<SWidget>, TFunctionRef<bool(TSharedRef<SWidget>)>)> FindWidgetByPredicate = 
			[&FindWidgetByPredicate](TSharedRef<SWidget> InWidget, TFunctionRef<bool(TSharedRef<SWidget>)> InPredicate)->TSharedPtr<SWidget>
			{
				if (InPredicate(InWidget))
				{
					return InWidget;
				}

				if (FChildren* Children = InWidget->GetAllChildren())
				{
					for (int32 Index = 0; Index < Children->Num(); ++Index)
					{
						const TSharedRef<SWidget> Widget = Children->GetChildAt(Index);
						if (TSharedPtr<SWidget> FoundWidget = FindWidgetByPredicate(Widget, InPredicate))
						{
							return FoundWidget;
						}
					}
				}

				return nullptr;
			};
		
		// Extract details view options button containing the menu to toggle settings for the view
		DetailsViewOptions = FindWidgetByPredicate(FilterWidget.ToSharedRef(), [](TSharedRef<SWidget> InWidget)
			{
				const TSharedPtr<FTagMetaData> MetaData = InWidget->GetMetaData<FTagMetaData>();
				return MetaData && MetaData->Tag.IsEqual(TEXT("ViewOptions"));
			}
		);
		
		// Extract details view sections selector containing the buttons to filter by section
		DetailsViewSections = FindWidgetByPredicate(FilterWidget.ToSharedRef(), [](TSharedRef<SWidget> InWidget)
			{
				const TSharedPtr<FTagMetaData> MetaData = InWidget->GetMetaData<FTagMetaData>();
				return MetaData && MetaData->Tag.IsEqual(TEXT("SectionView"));
			}
		);
	}

	if (!ensureMsgf(DetailsViewOptions.IsValid(), TEXT("SettingsEditor : Details view options widget cannot be found")))
	{
		DetailsViewOptions = SNullWidget::NullWidget;
	}

	if (!ensureMsgf(DetailsViewSections.IsValid(), TEXT("SettingsEditor : Details view sections widget cannot be found")))
	{
		DetailsViewSections = SNullWidget::NullWidget;
	}

	FCoreUObjectDelegates::OnObjectTransacted.AddSP(this, &SSettingsEditor::HandleObjectTransacted);

	// Register menu toolbar
	UToolMenus* ToolMenus = UToolMenus::Get();
	if (IsValid(ToolMenus) && !ToolMenus->IsMenuRegistered(SettingsEditorToolbarName))
	{
		UToolMenu* const SettingsToolbarMenu = ToolMenus->RegisterMenu(
			SettingsEditorToolbarName, NAME_None /* parent */, EMultiBoxType::SlimHorizontalToolBar
		);

		SettingsToolbarMenu->StyleName = "NoBackgroundViewportToolbar";
		SettingsToolbarMenu->bSeparateSections = false;
		SettingsToolbarMenu->AddDynamicSection(
			TEXT("Filters"), 
			FNewToolMenuDelegate::CreateStatic(&SSettingsEditor::FillSettingsFiltersSection)
		);
	}

	// Register sidebar
	{
		SidebarPanel = SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.Padding(4.f)
			.AutoHeight()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(0.0f)
				[
					SAssignNew(MenuToolbarBox, SBox)
					.MinDesiredHeight(24.f)
					[
						// Menu toolbar goes here
						SNullWidget::NullWidget
					]
				]
				+ SHorizontalBox::Slot()
				.FillWidth(1.0f)
				.VAlign(VAlign_Center)
				.Padding(0.f)
				[
					SAssignNew(SearchBox, SAssetSearchBox)
					.SuggestionListHeight(60.f)
					.ShowSearchHistory(false) // Only suggest, otherwise its confusing
					.DelayChangeNotificationsWhileTyping(true)
					.ToolTipText(FText::GetEmpty()) // Do not show any tooltip, it's showing when suggestions are listed...
					.HintText(this, &SSettingsEditor::GetSearchHintText)
					.OnTextChanged(this, &SSettingsEditor::OnSearchTextChanged)
					.OnTextCommitted(this, &SSettingsEditor::OnSearchTextCommitted)
					.OnAssetSearchBoxSuggestionFilter(this, &SSettingsEditor::OnAssetSearchSuggestionFilter)
					.OnAssetSearchBoxSuggestionChosen(this, &SSettingsEditor::OnAssetSearchSuggestionChosen)
					.OnKeyDownHandler(this, &SSettingsEditor::OnSearchKeyDownHandler)
					.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("SettingsEditorSearchBox")))
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(8.0f, 0.f, 0.f, 0.f)
				[
					// Details view cogwheel with options
					DetailsViewOptions.ToSharedRef()
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(8.0f, 0.0f, 0.0f, 0.0f)
				[
					// Import button
					SNew(SButton)
					.OnClicked(this, &SSettingsEditor::HandleImportButtonClicked)
					.Text(LOCTEXT("ImportAllButtonText", "Import All..."))
					.ToolTipText(LOCTEXT("ImportAllButtonTooltip", "Import all editor settings from a file on your computer"))
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(8.0f, 0.0f)
				[
					// Export button
					SNew(SButton)
					.OnClicked(this, &SSettingsEditor::HandleExportButtonClicked)
					.Text(LOCTEXT("ExportAllButtonText", "Export All..."))
					.ToolTipText(LOCTEXT("ExportAllButtonTooltip", "Export all editor settings to a file on your computer"))
				]
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(4.f)
			[
				// Details view sections selector
				DetailsViewSections.ToSharedRef()
			]
			+ SVerticalBox::Slot()
			.Padding(4.f)
			.FillHeight(1.f)
			[
				// settings area
				SNew(SOverlay)
				+ SOverlay::Slot()
				[
					SettingsView.ToSharedRef()
				]
				+ SOverlay::Slot()
				.Expose(CustomWidgetSlot)
			];

		const TSharedRef<SSidebarContainer> SidebarContainer = SNew(SSidebarContainer);
		Sidebar = SNew(SSidebar, SidebarContainer)
			.TabLocation(ESidebarTabLocation::Left)
			.InitialDrawerSize(0.20f)
			.DisablePin(true)
			.OnGetContent(FOnGetContent::CreateSPLambda(this, [this]()
			{
				return SidebarPanel.ToSharedRef();
			})
		);
		SidebarContainer->RebuildSidebar(Sidebar.ToSharedRef(), FSidebarState());
		
		// Create category tree later so everything is initialized when callback is triggered
		TSharedRef<SWidget> CategoryTreeWidget = SNew(SBorder)
		  .BorderImage(FAppStyle::Get().GetBrush("SettingsEditor.ListBorder"))
		  .Padding(8.f)
		  [
			  SAssignNew(CategoryTree, SSettingsEditorCategoryTree, SettingsContainer)
			  .InitialSelection(Model->GetSelectedSection())
			  .OnSelectionChanged(this, &SSettingsEditor::OnSectionSelectionChanged)
		  ];

		FSidebarDrawerConfig SidebarDrawerConfig;
		SidebarDrawerConfig.UniqueId = SettingsEditorSidebarId;
		SidebarDrawerConfig.ButtonText = LOCTEXT("SidebarButtonText", "Settings");
		SidebarDrawerConfig.ToolTipText = LOCTEXT("SidebarToolTipText", "Toggle the settings category tree sidebar when undocked");
		SidebarDrawerConfig.InitialState.bIsDocked = true;
		Sidebar->RegisterDrawer(MoveTemp(SidebarDrawerConfig));
		Sidebar->RegisterDrawerSection(
			SettingsEditorSidebarId,
			MakeShared<FSettingsEditorDrawerContent>(CategoryTreeWidget)
		);

		ChildSlot
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::Get().GetBrush("Brushes.Panel"))
			.Padding(0.f)
			[
				SidebarContainer
			]
		];
	}

	Model->OnSelectionChanged().AddSP(this, &SSettingsEditor::HandleModelSelectionChanged);
	HandleModelSelectionChanged();

	// Update toolbar menu
	UpdateToolbarMenu();
	if (ISettingsEditorModule* SettingsEditorModule = FModuleManager::GetModulePtr<ISettingsEditorModule>(TEXT("SettingsEditor")))
	{
		SettingsEditorModule->OnSearchTermsChanged().AddSP(this, &SSettingsEditor::OnSearchTermsChanged);
	}

	// Focus search box next tick
	FTSTicker::GetCoreTicker().AddTicker(
		FTickerDelegate::CreateSPLambda(
			this, 
			[this](float)
			{
				if (FSlateApplication::IsInitialized() && SearchBox.IsValid())
				{
					FSlateApplication::Get().SetKeyboardFocus(SearchBox, EFocusCause::SetDirectly);
				}
				return false; // Stop
			}
		)
	);
}

/* FNotifyHook interface
 *****************************************************************************/

void SSettingsEditor::NotifyPostChange( const FPropertyChangedEvent& PropertyChangedEvent, class FEditPropertyChain* PropertyThatChanged )
{
	if (PropertyChangedEvent.ChangeType != EPropertyChangeType::Interactive)
	{
		// Note while there could be multiple objects in the details panel, only one is ever edited at once.
		// There could be zero objects being edited in the FStructOnScope case.
		
		if (PropertyChangedEvent.GetNumObjectsBeingEdited() > 0)
		{
			UObject* ObjectBeingEdited = (UObject*)PropertyChangedEvent.GetObjectBeingEdited(0);
			const FProperty* MemberProperty = PropertyThatChanged->GetActiveMemberNode()->GetValue();
			const FProperty* ChangedProperty = PropertyChangedEvent.Property;

			HandleSettingsPropertyChanged(ObjectBeingEdited, MemberProperty, ChangedProperty);

			ModifiedObjectCache.Add(ObjectBeingEdited);
		}
	}
}

void SSettingsEditor::HandleSettingsPropertyChanged(UObject* ObjectBeingEdited, const FProperty* MemberProperty, const FProperty* ChangedProperty)
{
	// Get the section from the edited object.  We cannot use the selected section as multiple sections can be shown at once in the settings details panel.
	ISettingsSectionPtr Section = Model->GetSectionFromSectionObject(ObjectBeingEdited);
	{
		FString RelativePath;
		bool bIsSourceControlled = false;
		bool bIsNewFile = false;

		// Attempt to checkout the file automatically
		if (ObjectBeingEdited->GetClass()->HasAnyClassFlags(CLASS_DefaultConfig))
		{
			RelativePath = ObjectBeingEdited->GetDefaultConfigFilename();
			bIsSourceControlled = true;
		}
		else if (ObjectBeingEdited->GetClass()->HasAnyClassFlags(CLASS_Config))
		{
			RelativePath = ObjectBeingEdited->GetClass()->GetConfigName();
		}

		FString FullPath = FPaths::ConvertRelativePathToFull(RelativePath);

		if (!FPlatformFileManager::Get().GetPlatformFile().FileExists(*FullPath))
		{
			bIsNewFile = true;
		}

		if (!bIsSourceControlled || !SettingsHelpers::CheckOutOrAddFile(FullPath))
		{
			SettingsHelpers::MakeWritable(FullPath);
		}

		if (Section.IsValid())
		{
			RecordPreferenceChangedAnalytics(Section, MemberProperty);
		}

		static IConsoleVariable* NewPropertySaving = IConsoleManager::Get().FindConsoleVariable(TEXT("ini.UseNewPropertySaving"));
		bool bUseFullSectionSave = false;
		if (NewPropertySaving->GetInt() == 0)
		{
			// Determine if the Property is an Array or Array Element
			bool bIsArrayOrArrayElement = MemberProperty->IsA(FArrayProperty::StaticClass())
				|| MemberProperty->ArrayDim > 1
				|| ChangedProperty->GetOwner<FArrayProperty>();

			bool bIsSetOrSetElement = MemberProperty->IsA(FSetProperty::StaticClass())
				|| ChangedProperty->GetOwner<FSetProperty>();

			bool bIsMapOrMapElement = MemberProperty->IsA(FMapProperty::StaticClass())
				|| ChangedProperty->GetOwner<FMapProperty>();

			bUseFullSectionSave = bIsArrayOrArrayElement || bIsSetOrSetElement || bIsMapOrMapElement;
		}

		if (ObjectBeingEdited->GetClass()->HasAnyClassFlags(CLASS_DefaultConfig) && !bUseFullSectionSave)
		{
			if (!Section.IsValid() || Section->NotifySectionOnPropertyModified())
			{
				ObjectBeingEdited->UpdateSinglePropertyInConfigFile(MemberProperty, ObjectBeingEdited->GetDefaultConfigFilename());
			}
		}
		else if (Section.IsValid())
		{
			Section->Save();
		}
		// Some files being edited might have an array element, but they may also not have a corresponding section for inlined
		// external objects, for them if they're DefaultConfig, we update them here.
		else if (ObjectBeingEdited->GetClass()->HasAnyClassFlags(CLASS_DefaultConfig))
		{
			ObjectBeingEdited->TryUpdateDefaultConfigFile();
		}

		if (bIsNewFile && bIsSourceControlled)
		{
			SettingsHelpers::CheckOutOrAddFile(FullPath);
		}

		static const FName ConfigRestartRequiredKey = "ConfigRestartRequired";
		if (ChangedProperty->GetBoolMetaData(ConfigRestartRequiredKey) || MemberProperty->GetBoolMetaData(ConfigRestartRequiredKey))
		{
			OnApplicationRestartRequiredDelegate.ExecuteIfBound();
		}
	}
}

void SSettingsEditor::HandleObjectTransacted(UObject* InObject, const FTransactionObjectEvent& InTransactionEvent)
{
	if (InTransactionEvent.GetEventType() != ETransactionObjectEventType::UndoRedo)
	{
		return;
	}

	if (!InTransactionEvent.HasPropertyChanges())
	{
		return;
	}

	if (ModifiedObjectCache.Contains(InObject))
	{
		for (const FName& PropertyName : InTransactionEvent.GetChangedProperties())
		{
			if (const FProperty* MemberProperty = InObject->GetClass()->FindPropertyByName(PropertyName))
			{
				HandleSettingsPropertyChanged(InObject, MemberProperty, MemberProperty);
			}
		}
	}
}

/* SSettingsEditor implementation
 *****************************************************************************/

TWeakObjectPtr<UObject> SSettingsEditor::GetSelectedSettingsObject() const
{
	ISettingsSectionPtr SelectedSection = Model->GetSelectedSection();

	if (SelectedSection.IsValid())
	{
		return SelectedSection->GetSettingsObject();
	}

	return nullptr;
}


void SSettingsEditor::RecordPreferenceChangedAnalytics( ISettingsSectionPtr SelectedSection, const FProperty* ChangedProperty ) const
{
	// submit analytics data
	if(FEngineAnalytics::IsAvailable() && ChangedProperty != nullptr && ChangedProperty->GetOwnerClass() != nullptr)
	{
		TArray<FAnalyticsEventAttribute> EventAttributes;
		EventAttributes.Add(FAnalyticsEventAttribute(TEXT("PropertySection"), SelectedSection->GetName().ToString()));
		EventAttributes.Add(FAnalyticsEventAttribute(TEXT("PropertyClass"), ChangedProperty->GetOwnerClass()->GetName()));
		EventAttributes.Add(FAnalyticsEventAttribute(TEXT("PropertyName"), ChangedProperty->GetName()));

		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.PreferencesChanged"), EventAttributes);
	}
}

/* SSettingsEditor callbacks
 *****************************************************************************/

void SSettingsEditor::HandleModelSelectionChanged()
{
	// This callback can trigger on unregister during shutdown, simply return in this case
	if (!FSlateApplication::IsInitialized())
	{
		return;
	}

	if (const ISettingsSectionPtr SelectedSection = Model->GetSelectedSection())
	{
		// show settings widget
		if (const TSharedPtr<SWidget> CustomWidget = SelectedSection->GetCustomWidget().Pin())
		{
			CustomWidgetSlot->AttachWidget( CustomWidget.ToSharedRef() );
		}
		else
		{
			CustomWidgetSlot->AttachWidget( SNullWidget::NullWidget );
		}

		bShowingAllSettings = false;

		// Sync tree view with selection
		CategoryTree->SetSelection(SelectedSection);
	}
	else
	{
		bShowingAllSettings = true;
		CustomWidgetSlot->AttachWidget( SNullWidget::NullWidget );

		// Sync tree view with selection
		CategoryTree->SetSelection(/** Section */nullptr);
	}

	// clear the global search terms when selecting a specific category
	SearchKey.Reset();
	bAutoDetectKeybind = false;
	SetSearchFilter(FText::GetEmpty());

	// Set all settings object to preview them,
	// they will get filtered base on the active section selection in the customization
	{
		const bool bInit = SettingsView->GetSelectedObjects().IsEmpty();

		TArray<UObject*> SettingsObjects;
		CategoryTree->ForEachSection([&SettingsObjects](const TSharedPtr<ISettingsSection>& Section)
			{
				if (UObject* SettingObject = Section->GetSettingsObject().Get())
				{
					SettingsObjects.Add(SettingObject);
				}
				
				return true; // Continue
			}
		);
		SettingsView->SetObjects(SettingsObjects);
		
		// Settings view will be populated with all settings objects and hide them internally through a customization
		// Show sidebar if more than 1 settings object
		const bool bDockSidebar = SettingsObjects.Num() > 1;
		if (bInit && bDockSidebar != Sidebar->IsDrawerDocked(SettingsEditorSidebarId))
		{
			Sidebar->SetDrawerDocked(SettingsEditorSidebarId, bDockSidebar);
		}
	}
}

EVisibility SSettingsEditor::HandleSettingsBoxVisibility() const
{
	ISettingsSectionPtr SelectedSection = Model->GetSelectedSection();

	if (SelectedSection.IsValid() || bShowingAllSettings)
	{
		return EVisibility::Visible;
	}

	return EVisibility::Hidden;
}

FReply SSettingsEditor::HandleExportButtonClicked()
{
	const FString DefaultFileName = FString::Printf(TEXT("%s Settings Backup %s.ini"), *Model->GetSettingsContainer()->GetName().ToString(), *FDateTime::Now().ToString(TEXT("%Y-%m-%d %H%M%S")));

	const TSharedPtr<SWindow> ParentWindow = FSlateApplication::Get().FindWidgetWindow(AsShared());
	const void* ParentWindowHandle = ParentWindow.IsValid() && ParentWindow->GetNativeWindow().IsValid() ? ParentWindow->GetNativeWindow()->GetOSWindowHandle() : nullptr;

	TArray<FString> OutSaveFiles;
	if (FDesktopPlatformModule::Get()->SaveFileDialog(ParentWindowHandle, 
		LOCTEXT("ExportSettingsDialogTitle", "Export settings...").ToString(), 
		FPaths::GetPath(GEditorPerProjectIni),
		DefaultFileName, 
		TEXT("Config files (*.ini)|*.ini"), 
		EFileDialogFlags::None, 
		OutSaveFiles))
	{
		const FString DestinationFilePath = OutSaveFiles[0];
		if (GConfig->FindBranch(*DestinationFilePath, DestinationFilePath))
		{
			SSettingsSectionHeader::ShowNotification(LOCTEXT("ExportSettingsExistingFileFailure", "Export settings failed. Cannot use a file that is already part of the config system."), SNotificationItem::CS_Fail);
			return FReply::Handled();
		}

		FString DestinationFolder;
		FString DestinationExtension;
		FString DestinationFilenameWithoutExtension;
		FPaths::Split(DestinationFilePath, DestinationFolder, DestinationFilenameWithoutExtension, DestinationExtension);

		// Exports will be done to a temp folder so we can be aware of any extra files generated via custom Export delegates within the destination folder
		const FString TempDestinationFolder = FPaths::ProjectSavedDir() / TEXT("TempExport");

		constexpr bool bDeleteDirectoryRequireExists = false;
		constexpr bool bDeleteDirectoryRecursively = true;

		// If the TempExport folder already exists, prompt the user to delete it.
		// This should really only happen if there was a crash during export or the user manually created it.
		if (FPaths::DirectoryExists(TempDestinationFolder))
		{
			if (FMessageDialog::Open(EAppMsgType::OkCancel,
				LOCTEXT("ExportSettingsDeleteTempFolderMsg", "TempExport settings folder already exists within project's Saved folder, possibly from a previous failed export. Do you wish to delete this folder so this export may continue?"),
				LOCTEXT("ExportSettingsDeleteTempFolderTitle", "Delete Folder?")) == EAppReturnType::Ok)
			{
				IFileManager::Get().DeleteDirectory(*TempDestinationFolder, bDeleteDirectoryRequireExists, bDeleteDirectoryRecursively);
			}
			else
			{
				SSettingsSectionHeader::ShowNotification(LOCTEXT("ExportSettingsExistingTempFolderFailure", "Export settings failed. TempExport folder already exists under project Saved folder."), SNotificationItem::CS_Fail);
				return FReply::Handled();
			}
		}

		FConfigBranch& TempBranch = GConfig->AddNewBranch(DestinationFilePath);

		// A temp destination file is used to accomodate custom Export delegates that use intermediate files that are then copied into the destination file
		const FString TempDestinationFilePath = TempDestinationFolder / DestinationFilenameWithoutExtension + TEXT(".") + DestinationExtension;

		TArray<ISettingsCategoryPtr> Categories;
		SettingsContainer->GetCategories(Categories);

		for (const ISettingsCategoryPtr& Category : Categories)
		{
			TArray<ISettingsSectionPtr> Sections;
			Category->GetSections(Sections);
			Sections.Sort([](const ISettingsSectionPtr& First, const ISettingsSectionPtr& Second)
			{
				return First->GetDisplayName().CompareTo(Second->GetDisplayName()) < 0;
			});

			for (const ISettingsSectionPtr& Section : Sections)
			{
				if (!Section->Export(*TempDestinationFilePath))
				{
					continue;
				}

				TempBranch.InMemoryFile.Combine(TempDestinationFilePath, /** HandleSymbolCommands */false);
				IFileManager::Get().Delete(*TempDestinationFilePath);

				TArray<FString> AdditionalFiles;
				IFileManager::Get().FindFiles(AdditionalFiles, *TempDestinationFolder);

				// Copy any additional files to the destination file's folder with a unique name based on the destination file path and the section it was in
				for (const FString& AdditionalFile : AdditionalFiles)
				{
					FString AdditionalFilePath = TempDestinationFolder / AdditionalFile;
					FString NewAdditionalFilePath = FString::Printf(TEXT("%s/%s_%s_%s"), *DestinationFolder, *DestinationFilenameWithoutExtension, *Section->GetDisplayName().ToString(), *AdditionalFile);
					IFileManager::Get().Copy(*NewAdditionalFilePath, *AdditionalFilePath);
					IFileManager::Get().Delete(*AdditionalFilePath);
				}
			}
		}

		GConfig->Flush(true, *DestinationFilePath);

		IFileManager::Get().DeleteDirectory(*TempDestinationFolder, bDeleteDirectoryRequireExists, bDeleteDirectoryRecursively);

		SSettingsSectionHeader::ShowNotification(LOCTEXT("ExportSettingsSuccess", "Export settings succeeded"), SNotificationItem::CS_Success, DestinationFilePath);
		return FReply::Handled();
	}

	SSettingsSectionHeader::ShowNotification(LOCTEXT("ExportSettingsFailure", "Export settings failed"), SNotificationItem::CS_Fail);
	return FReply::Handled();
}

FReply SSettingsEditor::HandleImportButtonClicked()
{
	const TSharedPtr<SWindow> ParentWindow = FSlateApplication::Get().FindWidgetWindow(AsShared());
	const void* ParentWindowHandle = ParentWindow.IsValid() && ParentWindow->GetNativeWindow().IsValid() ? ParentWindow->GetNativeWindow()->GetOSWindowHandle() : nullptr;

	TArray<FString> OutFiles;
	if (FDesktopPlatformModule::Get()->OpenFileDialog(ParentWindowHandle, 
		LOCTEXT("ImportSettingsDialogTitle", "Import settings...").ToString(),
		FPaths::GetPath(GEditorPerProjectIni), 
		TEXT(""), 
		TEXT("Config files (*.ini)|*.ini"), 
		EFileDialogFlags::None, 
		OutFiles))
	{
		const FString FilePath = OutFiles[0];

		TArray<ISettingsCategoryPtr> Categories;
		SettingsContainer->GetCategories(Categories);
		for (const ISettingsCategoryPtr& Category : Categories)
		{
			TArray<ISettingsSectionPtr> Sections;
			Category->GetSections(Sections);

			for (const ISettingsSectionPtr& Section : Sections)
			{
				Section->Import(*FilePath);
			}
		}

		SSettingsSectionHeader::ShowNotification(LOCTEXT("ImportSettingsSuccess", "Import settings succeeded"), SNotificationItem::CS_Success);
	}
	else
	{
		SSettingsSectionHeader::ShowNotification(LOCTEXT("ImportSettingsFailure", "Import settings failed"), SNotificationItem::CS_Fail);
	}

	return FReply::Handled();
}

FText SSettingsEditor::GetSearchHintText() const
{
	// Custom hint for when we are listening for key presses
	static const FText KeyBindSearch = LOCTEXT("KeyBindSearch", "(Press any key(s) to search)");
	if (bAutoDetectKeybind)
	{
		return KeyBindSearch;
	}

	FText SectionText;
	if (const TSharedPtr<ISettingsSection> SelectedSection = Model->GetSelectedSection())
	{
		SectionText = SelectedSection->GetDisplayName();
	}
	else
	{
		SectionText = LOCTEXT("SettingsAllSection", "all settings");
	}
	
	if (SearchKey.IsEmpty())
	{
		static const FText HintFormat = LOCTEXT("SearchHintText", "Search in {0}...");
		return FText::Format(HintFormat, SectionText);
	}
	else
	{
		static const FText AdvancedHintFormat = LOCTEXT("SearchAdvancedHintText", "Search in {0} for {1}...");
		return FText::Format(AdvancedHintFormat, SectionText, FText::FromString(SearchKey.ToLower()));
	}
}

void SSettingsEditor::OnSearchTextChanged(const FText& InText)
{
	if (SettingsView.IsValid())
	{
		SetSearchFilter(InText);
	}
}

void SSettingsEditor::OnSearchTextCommitted(const FText& InText, ETextCommit::Type InType)
{
	if (SettingsView.IsValid() && InType == ETextCommit::Type::OnCleared)
	{
		SetSearchFilter(FText::GetEmpty());
	}
}

void SSettingsEditor::OnAssetSearchSuggestionFilter(const FText& InText, TArray<FAssetSearchBoxSuggestion>& OutSuggestion, FText& OutText)
{
	const ISettingsEditorModule* SettingsEditorModule = FModuleManager::GetModulePtr<ISettingsEditorModule>(TEXT("SettingsEditor"));
	if (!SettingsEditorModule || !SettingsContainer)
	{
		return;
	}

	const TMap<FString, ISettingsEditorModule::FSearchTerm>* SearchTerms = SettingsEditorModule->FindSearchTerms(SettingsContainer->GetName());
	if (!SearchTerms)
	{
		return;
	}

	static const FText SettingsCategory = LOCTEXT("SettingsEditor", "Settings");

	// Do not append search key to the suggestion again
	FString ActiveSearch;
	if (!SearchKey.IsEmpty())
	{
		ActiveSearch = FText::FormatOrdered(TokenCombineFormat, FText::FromString(SearchKey), InText).ToString();
	}
	else
	{
		ActiveSearch = InText.ToString();
	}

	const FName SelectedSectionName = GetSelectedSectionName();
	for (const TPair<FString, ISettingsEditorModule::FSearchTerm>& SearchTerm : *SearchTerms)
	{
		if (!SearchTerm.Value.PassesSectionFilter(SelectedSectionName))
		{
			continue;
		}

		// Suggest search term first, then suggest search term + search value 
		// eg: typing "cv" will show "cvar", and typing "cvar" will show "cvar R.VSync"
		const bool bActiveSearchContainsSearchTermKey = ActiveSearch.Contains(SearchTerm.Key);
		const bool bSearchTermContainsActiveSearch = SearchTerm.Key.Contains(ActiveSearch);
		if (bActiveSearchContainsSearchTermKey || bSearchTermContainsActiveSearch)
		{
			FString SearchBase = !SearchKey.IsEmpty() ? TEXT("") : SearchTerm.Key.ToLower();
			FString SearchSuggestion = SearchBase;
			FText SearchSuggestionLabel;
			if (!SearchBase.IsEmpty() && !bActiveSearchContainsSearchTermKey)
			{
				SearchSuggestionLabel = FText::FromString(SearchBase);
				OutSuggestion.Add(FAssetSearchBoxSuggestion(SearchSuggestion, SearchSuggestionLabel, SettingsCategory));
			}

			FString SearchCopy = ActiveSearch;
			SearchCopy.RemoveFromStart(SearchTerm.Key.ToLower() + TEXT(" "));

			for (const FString& SearchValue : SearchTerm.Value.Values)
			{
				// Allow partial search (eg: "collect" will list "gc.collecteveryframe")
				if (bSearchTermContainsActiveSearch || SearchCopy.IsEmpty() || SearchValue.Contains(SearchCopy))
				{
					SearchSuggestion = SearchBase + TEXT(" \"") + SearchValue + TEXT("\"");
					SearchSuggestionLabel = FText::FromString(SearchSuggestion);
					OutSuggestion.Add(FAssetSearchBoxSuggestion(SearchSuggestion, SearchSuggestionLabel, SettingsCategory));
				}
			}
		}
	}

	// Clear suggestions when we already have it filled
	if (OutSuggestion.Num() == 1 && OutSuggestion[0].SuggestionString == InText.ToString())
	{
		OutSuggestion.Reset();
	}
}

FText SSettingsEditor::OnAssetSearchSuggestionChosen(const FText& InText, const FString& InSuggestion)
{
	FString CurrentSearch = InText.ToString();

	// Trim the start, eg: "keybind ctrl" becomes "ctrl"
	// This is needed because the "..." used for strict search do not match with the current search (eg: keybind "ctrl" vs keybind ctrl)
	int32 SpaceIndex;
	if (CurrentSearch.FindChar(' ', SpaceIndex))
	{
		CurrentSearch = CurrentSearch.RightChop(SpaceIndex + 1);
	}

	if (!InSuggestion.Contains(CurrentSearch))
	{
		for (const auto& Char : InSuggestion)
		{
			if (!CurrentSearch.IsEmpty() && CurrentSearch[0] == Char)
			{
				CurrentSearch.RemoveAt(0);
			}
			else
			{
				break;
			}
		}
		
		CurrentSearch = InSuggestion + CurrentSearch;
	}
	else
	{
		CurrentSearch = InSuggestion;
	}

	return FText::FromString(CurrentSearch);
}

void SSettingsEditor::FillSettingsFiltersSection(UToolMenu* InMenu)
{
	if (!InMenu)
	{
		return;
	}

	const USettingsEditorMenuContext* Context = InMenu->FindContext<USettingsEditorMenuContext>();
	if (!Context)
	{
		return;
	}

	const TSharedPtr<SSettingsEditor> SettingsEditor = Context->GetSettingsEditor();
	if (!SettingsEditor)
	{
		return;
	}

	const ISettingsEditorModule* SettingsEditorModule = FModuleManager::GetModulePtr<ISettingsEditorModule>(TEXT("SettingsEditor"));
	if (!SettingsEditorModule || !SettingsEditor->SettingsContainer)
	{
		return;
	}

	const TMap<FString, ISettingsEditorModule::FSearchTerm>* SearchTerms = SettingsEditorModule->FindSearchTerms(SettingsEditor->SettingsContainer->GetName());
	if (!SearchTerms)
	{
		return;
	}

	FToolMenuSection& FiltersSection = InMenu->FindOrAddSection("Filters", LOCTEXT("FiltersLabel", "Filters"));
	for (const TPair<FString, ISettingsEditorModule::FSearchTerm>& SearchTerm : *SearchTerms)
	{
		FToolMenuEntry MenuEntry = CreateFiltersCheckboxEntry(SettingsEditorModule, Context, SearchTerm.Key, SearchTerm.Value.Values);

		// Customize icon
		if (!SearchTerm.Value.IconStyleSetName.IsNone() && !SearchTerm.Value.IconStyleName.IsNone())
		{
			MenuEntry.Icon = FSlateIcon(SearchTerm.Value.IconStyleSetName, SearchTerm.Value.IconStyleName);
		}
		else
		{
			MenuEntry.Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Filter");
		}

		// Customize label
		if (!SearchTerm.Value.Label.IsEmpty())
		{
			MenuEntry.Label = SearchTerm.Value.Label;
		}

		FiltersSection
			.AddEntry(MenuEntry)
			.SetShowInToolbarTopLevel(true);
	}
}

FToolMenuEntry SSettingsEditor::CreateFiltersCheckboxEntry(const ISettingsEditorModule* InSettingsEditorModule, const USettingsEditorMenuContext* InContext, const FString& InKey, const TSet<FString>& InValues)
{
	const TSharedPtr<SSettingsEditor> SettingsEditor = InContext->GetSettingsEditor();
	checkf(SettingsEditor.IsValid(), TEXT("SettingsEditor must be valid"));
	checkf(InSettingsEditorModule != nullptr, TEXT("SettingsEditorModule must be valid"));

	FUIAction FilterAction;
	FilterAction.ExecuteAction = FExecuteAction::CreateSP(SettingsEditor.Get(), &SSettingsEditor::ToggleSearchFilterAction, InKey);
	FilterAction.GetActionCheckState = FGetActionCheckState::CreateSP(SettingsEditor.Get(), &SSettingsEditor::IsSearchFilterActive, InKey);
	FilterAction.IsActionVisibleDelegate = FIsActionButtonVisible::CreateSP(SettingsEditor.Get(), &SSettingsEditor::IsSearchFilterVisible, InKey);

	return FToolMenuEntry::InitToolBarButton(
		FName(InKey),
		FilterAction,
		FText::FromString(InKey),
		TAttribute<FText>::CreateSP(SettingsEditor.Get(), &SSettingsEditor::GetSearchFilterTooltip, InSettingsEditorModule, InKey),
		TAttribute<FSlateIcon>(),
		EUserInterfaceActionType::ToggleButton
	);
}

void SSettingsEditor::ToggleSearchFilterAction(FString InFilter)
{
	if (IsSearchFilterActive(InFilter) == ECheckBoxState::Unchecked)
	{
		SearchKey = InFilter;

		// Custom use case to detect which keys have been pressed by user and perform a search on those
		static const FText KeybindKey = NSLOCTEXT("InputBindingEditor", "SearchKeyKeybind", "KeyBind:");
		if (SearchKey == KeybindKey.ToString())
		{
			bAutoDetectKeybind = true;
		}
	}
	else
	{
		bAutoDetectKeybind = false;
		SearchKey.Reset();
	}

	SetSearchFilter(FText::GetEmpty());
	FSlateApplication::Get().SetKeyboardFocus(SearchBox, EFocusCause::SetDirectly);
}

ECheckBoxState SSettingsEditor::IsSearchFilterActive(FString InFilter)
{
	const bool bSearchFilterActive = SearchKey.Equals(InFilter, ESearchCase::CaseSensitive);
	return bSearchFilterActive ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void SSettingsEditor::SetSearchFilter(const FText& InSearch, bool bInStrict)
{
	FText SearchText = InSearch;
	if (bInStrict)
	{
		SearchText = FText::FormatOrdered(TokenDelimiter, SearchText);
	}

	if (!SearchKey.IsEmpty())
	{
		SearchText = FText::FormatOrdered(TokenCombineFormat, FText::FromString(SearchKey), SearchText);
	}

	SearchBox->SetText(InSearch);
	SettingsView->SetSearchText(SearchText);
}

bool SSettingsEditor::IsSearchFilterVisible(FString InSearchTerm) const
{
	ISettingsEditorModule* SettingsEditorModule = FModuleManager::GetModulePtr<ISettingsEditorModule>(TEXT("SettingsEditor"));
	if (!SettingsEditorModule || !SettingsContainer || !Model)
	{
		return false;
	}

	const ISettingsEditorModule::FSearchTerm* SearchTerm = SettingsEditorModule->FindSearchTerm(SettingsContainer->GetName(), InSearchTerm);
	if (!SearchTerm || !SearchTerm->bShowFilter)
	{
		return false;
	}

	const FName SectionName = GetSelectedSectionName();
	return SearchTerm->PassesSectionFilter(SectionName);
}

FName SSettingsEditor::GetSelectedSectionName() const
{
	FName SectionName = NAME_None;
	if (!Model)
	{
		return SectionName;
	}
	const TSharedPtr<ISettingsSection>& SelectedSection = Model->GetSelectedSection();
	if (!SelectedSection)
	{
		return SectionName;
	}

	SectionName = SelectedSection->GetName();
	return SectionName;
}

FText SSettingsEditor::GetSearchFilterTooltip(const ISettingsEditorModule* InSettingsEditorModule, FString InKey) const
{
	static const FTextFormat TooltipFormat = INVTEXT("{0}\n{1}");
	static const FText StatusTooltipText = LOCTEXT("DisabledSearchFilterToolTip", "Filter disabled since it is not relevant to the selected section");
	static const FText DefaultTooltipText = LOCTEXT("SearchFilterToolTip", "Toggles search filter to only show entries relevant to the selected context");

	FText TooltipText = DefaultTooltipText;
	if (SettingsContainer && InSettingsEditorModule != nullptr)
	{
		if (const ISettingsEditorModule::FSearchTerm* SearchTerms = InSettingsEditorModule->FindSearchTerm(SettingsContainer->GetName(), InKey))
		{
			if (!SearchTerms->Tooltip.IsEmpty())
			{
				TooltipText = SearchTerms->Tooltip;
			}
		}
	}

	const bool bIsFilterEnabled = IsSearchFilterVisible(InKey);
	return bIsFilterEnabled ? TooltipText : FText::Format(TooltipFormat, TooltipText, StatusTooltipText);
}

void SSettingsEditor::OnSectionSelectionChanged(TSharedPtr<ISettingsSection> InSection)
{
	Model->SelectSection(InSection);
}

FReply SSettingsEditor::OnSearchKeyDownHandler(const FGeometry& InGeometry, const FKeyEvent& InEvent)
{
	if (bAutoDetectKeybind)
	{
		const FInputChord InputChord(InEvent.GetKey(), InEvent.IsShiftDown(), InEvent.IsControlDown(), InEvent.IsAltDown(), InEvent.IsCommandDown());

		// Once a non modifier key is pressed, stop listening
		if (!InEvent.GetKey().IsModifierKey())
		{
			// Clear focus to avoid key to be processed like backspace
			FSlateApplication::Get().ClearKeyboardFocus(EFocusCause::Cleared);

			// Refocus search box
			FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateSPLambda(this, [this](float)->bool
				{
					if (FSlateApplication::IsInitialized())
					{
						FSlateApplication::Get().SetKeyboardFocus(SearchBox, EFocusCause::SetDirectly);
					}
					return false; // Stop
				})
			);
		}

		// Perform a strict search here
		constexpr bool bStrictSearch = true;
		SetSearchFilter(InputChord.GetInputText(), bStrictSearch);
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

void SSettingsEditor::OnSearchTermsChanged(FName InContainerName, const FString& InKey)
{
	if (MenuToolbarUpdateHandle.IsValid())
	{
		return;
	}

	MenuToolbarUpdateHandle = FTSTicker::GetCoreTicker().AddTicker(
		FTickerDelegate::CreateSPLambda(this, [this](float)
		{
			UpdateToolbarMenu();
			MenuToolbarUpdateHandle.Reset();
			return false;
		})
	);
}

void SSettingsEditor::UpdateToolbarMenu()
{
	if (UToolMenus* ToolMenus = UToolMenus::Get())
	{
		USettingsEditorMenuContext* MenuContext = NewObject<USettingsEditorMenuContext>();
		MenuContext->SettingsEditorWeak = SharedThis(this);
		const TSharedRef<SWidget> MenuWidget = ToolMenus->GenerateWidget(SettingsEditorToolbarName, FToolMenuContext(MenuContext));
		MenuToolbarBox->SetContent(MenuWidget);
	}
}

bool SSettingsEditor::HandleSettingsViewEnabled() const
{
	ISettingsSectionPtr SelectedSection = Model->GetSelectedSection();

	return (SelectedSection.IsValid() && SelectedSection->CanEdit()) || bShowingAllSettings;
}


EVisibility SSettingsEditor::HandleSettingsViewVisibility() const
{
	ISettingsSectionPtr SelectedSection = Model->GetSelectedSection();

	if (bShowingAllSettings || (SelectedSection.IsValid() && SelectedSection->GetSettingsObject().IsValid()))
	{
		return EVisibility::Visible;
	}

	return EVisibility::Hidden;
}


#undef LOCTEXT_NAMESPACE
