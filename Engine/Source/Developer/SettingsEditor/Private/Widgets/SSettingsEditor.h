// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/Ticker.h"
#include "ISettingsCategory.h"
#include "ISettingsContainer.h"
#include "ISettingsEditorModel.h"
#include "ISettingsEditorModule.h"
#include "ISettingsSection.h"
#include "Layout/Visibility.h"
#include "Misc/NotifyHook.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SOverlay.h"

class FTransactionObjectEvent;
class IDetailsView;
class SAssetSearchBox;
class SBox;
class SScrollBox;
class SSettingsEditorCategoryTree;
class SSidebar;
class SSidebarContainer;
class USettingsEditorMenuContext;
class UToolMenu;
enum class ECheckBoxState : uint8;
struct FAssetSearchBoxSuggestion;
struct FPropertyChangedEvent;
struct FToolMenuEntry;

/**
 * Implements an editor widget for settings.
 */
class SSettingsEditor
	: public SCompoundWidget
	, public FNotifyHook
{
public:
	/** Menu toolbar Identifier */
	static FLazyName SettingsEditorToolbarName;
	static FLazyName SettingsEditorSidebarId;

	SLATE_BEGIN_ARGS(SSettingsEditor) { }
		SLATE_EVENT( FSimpleDelegate, OnApplicationRestartRequired )
	SLATE_END_ARGS()

public:

	/** Destructor. */
	virtual ~SSettingsEditor() override;

public:

	/**
	 * Constructs the widget.
	 *
	 * @param InArgs The Slate argument list.
	 * @param InModel The view model.
	 */
	void Construct( const FArguments& InArgs, const ISettingsEditorModelRef& InModel );

public:

	// FNotifyHook interface

	virtual void NotifyPostChange( const FPropertyChangedEvent& PropertyChangedEvent, class FEditPropertyChain* PropertyThatChanged ) override;

protected:


	/**
	 * Gets the settings object of the selected section, if any.
	 *
	 * @return The settings object.
	 */
	TWeakObjectPtr<UObject> GetSelectedSettingsObject() const;


	/**
	 * Reports a preference changed event to the analytics system.
	 *
	 * @param SelectedSection The currently selected settings section.
	 * @param ChangedProperty The member property that changed.
	 */
	void RecordPreferenceChangedAnalytics( ISettingsSectionPtr SelectedSection, const FProperty* ChangedProperty) const;

private:
	static const FTextFormat TokenCombineFormat;
	static const FTextFormat TokenDelimiter;

	static void FillSettingsFiltersSection(UToolMenu* InMenu);
	static FToolMenuEntry CreateFiltersCheckboxEntry(const ISettingsEditorModule* InSettingsEditorModule, const USettingsEditorMenuContext* InContext, const FString& InKey, const TSet<FString>& InValues);

	void HandleSettingsPropertyChanged(UObject* InObjectBeingEdited, const FProperty* InMemberProperty, const FProperty* InChangedProperty);
	void HandleObjectTransacted(UObject* InObject, const FTransactionObjectEvent& InTransactionEvent);

	/** Callback for changing the selected settings section. */
	void HandleModelSelectionChanged();

	/** Callback for determining the visibility of the settings box. */
	EVisibility HandleSettingsBoxVisibility() const;

	/** Callback for checking whether the settings view is enabled. */
	bool HandleSettingsViewEnabled() const;

	/** Callback for determining the visibility of the settings view. */
	EVisibility HandleSettingsViewVisibility() const;

	/** Callback for exporting all sections as a single ini. */
	FReply HandleExportButtonClicked();

	/** Callback for importing all sections from a single ini. */
	FReply HandleImportButtonClicked();

	FText GetSearchHintText() const;
	void OnSearchTextChanged(const FText& InText);
	void OnSearchTextCommitted(const FText& InText, ETextCommit::Type InType);
	void OnAssetSearchSuggestionFilter(const FText& InText, TArray<FAssetSearchBoxSuggestion>& OutSuggestion, FText& OutText);
	FText OnAssetSearchSuggestionChosen(const FText& InText, const FString& InSuggestion);

	void ToggleSearchFilterAction(FString InFilter);
	ECheckBoxState IsSearchFilterActive(FString InFilter);
	void SetSearchFilter(const FText& InSearch, bool bInStrict = false);
	bool IsSearchFilterVisible(FString InSearchTerm) const;
	FName GetSelectedSectionName() const;
	FText GetSearchFilterTooltip(const ISettingsEditorModule* InSettingsEditorModule, FString InKey) const;

	void OnSectionSelectionChanged(TSharedPtr<ISettingsSection> InSection);
	FReply OnSearchKeyDownHandler(const FGeometry& InGeometry, const FKeyEvent& InEvent);
	void OnSearchTermsChanged(FName InContainerName, const FString& InKey);

	void UpdateToolbarMenu();

	/** Holds the settings category tree. */
	TSharedPtr<SSettingsEditorCategoryTree> CategoryTree;

	/** Holds the overlay slot for custom widgets. */
	SOverlay::FOverlaySlot* CustomWidgetSlot = nullptr;

	/** Holds a pointer to the view model. */
	ISettingsEditorModelPtr Model;

	/** Holds the settings container. */
	ISettingsContainerPtr SettingsContainer;

	/** Holds the details view. */
	TSharedPtr<IDetailsView> SettingsView;

	/** Delegate called when this settings editor requests that the user be notified that the application needs to be restarted for some setting changes to take effect */
	FSimpleDelegate OnApplicationRestartRequiredDelegate;
	
	/** Are we showing all settings at once */
	bool bShowingAllSettings = false;

	/** Actual search box with suggestions */
	TSharedPtr<SAssetSearchBox> SearchBox;
	
	/** Used to detect when to listen for key down events */
	bool bAutoDetectKeybind = false;
	
	/** Box for the horizontal menu toolbar */
	TSharedPtr<SBox> MenuToolbarBox = nullptr;
	
	/** Handle for when an update is planned */
	FTSTicker::FDelegateHandle MenuToolbarUpdateHandle;

	/** Used to store active search key */
	FString SearchKey;

	/** 
	 * Cache of modified objects added to when the user changes properties in the details view.
	 * Used to filter objects in a FCoreUObjectDelegates::OnObjectTransacted handler and only handle 
	 * undo/redo events on objects that we've observed the user change a property of in the details view.
	 * Note that these objects can be settings objects as well as sub-objects of a settings object.
	 */
	TSet<TWeakObjectPtr<UObject>> ModifiedObjectCache;

	/** Sidebar (category tree) and sidebar panel (details view) */
	TSharedPtr<SSidebar> Sidebar;
	TSharedPtr<SWidget> SidebarPanel;
};
