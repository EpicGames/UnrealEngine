// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SSettingsEditorCategoryItem.h"

#include "Internationalization/Internationalization.h"
#include "ISettingsCategory.h"
#include "ISettingsSection.h"
#include "Styling/AppStyle.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SExpanderArrow.h"

#define LOCTEXT_NAMESPACE "SettingsEditorCategoryItem"

using namespace UE::SettingsEditor::Private;

void SSettingsEditorCategoryItem::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwner, const TSharedRef<FTreeItem>& InItem)
{
	ItemWeak = InItem;

	// Figure out which style to use
	const auto bIsCategory = InItem->Category.IsValid() && !InItem->Section.IsValid();
	const auto bIsAllCategory = !InItem->Category.IsValid() && !InItem->Section.IsValid();

	FText Label;
	FText ToolTipText;
	FName BorderPaddingName;
	if (bIsAllCategory)
	{
		Label = LOCTEXT("AllPropertiesLink", "All Settings");
		ToolTipText = LOCTEXT("AllPropertiesLink_Tooltip", "Show all settings");
		BorderPaddingName = TEXT("CategoryTreeItem.Root.AllBackgroundPadding");
	}
	else if (bIsCategory)
	{
		Label = InItem->Category->GetDisplayName();
		ToolTipText = InItem->Category->GetDescription();
		BorderPaddingName = TEXT("CategoryTreeItem.Root.BackgroundPadding");
	}
	else
	{
		Label = InItem->Section->GetDisplayName();
		ToolTipText = InItem->Section->GetDescription();
		BorderPaddingName = TEXT("CategoryTreeItem.BackgroundPadding");
	}

	this->ChildSlot
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Fill)
		[
			SNew(SExpanderArrow, SharedThis(this))
			.IndentAmount(0)
		]
		+ SHorizontalBox::Slot()
		.FillWidth(1)
		[
			SNew(SBorder)
			.Padding(FAppStyle::Get().GetMargin(BorderPaddingName))
			.BorderImage(FStyleDefaults::GetNoBrush())
			[
				SNew(SHorizontalBox)
				// Category name
				+SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.Padding(FAppStyle::Get().GetFloat("CategoryTreeItem.PaddingAmount"))
				[
					SNew(STextBlock)
					.Text(Label)
					.ToolTipText(ToolTipText)
					.TextStyle(FAppStyle::Get(), (bIsCategory || bIsAllCategory) ? "CategoryTreeItem.Root.Text" : "CategoryTreeItem.Text")
				]
			]
		]
	];

	STableRow<TSharedPtr<FTreeItem>>::ConstructInternal(
		STableRow::FArguments()
			.Style(&FAppStyle::Get(), "SimpleTableView.Row")
			.ShowSelection(!bIsCategory),
		InOwner
	);
}

#undef LOCTEXT_NAMESPACE
