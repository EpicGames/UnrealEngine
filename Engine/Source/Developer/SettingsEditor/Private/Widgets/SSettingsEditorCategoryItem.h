// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/Views/ITableRow.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"

class ISettingsSection;
class ISettingsCategory;
class STableViewBase;

namespace UE::SettingsEditor::Private
{
	struct FTreeItem
	{
		TSharedPtr<ISettingsCategory> Category;
		TSharedPtr<ISettingsSection> Section;
		TArray<TSharedPtr<FTreeItem>> Children;
	};
}

/**
 * Widget that represents a single item in SSettingsEditorCategoryTree
 */
class SSettingsEditorCategoryItem : public STableRow<TSharedPtr<UE::SettingsEditor::Private::FTreeItem>>
{
public:
	SLATE_BEGIN_ARGS(SSettingsEditorCategoryItem)
	{}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwner, const TSharedRef<UE::SettingsEditor::Private::FTreeItem>& InItem);

private:
	TWeakPtr<UE::SettingsEditor::Private::FTreeItem> ItemWeak;
};