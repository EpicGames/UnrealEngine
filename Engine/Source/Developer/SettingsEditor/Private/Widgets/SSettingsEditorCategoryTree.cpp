// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SSettingsEditorCategoryTree.h"

#include "Containers/Ticker.h"
#include "Internationalization/Internationalization.h"
#include "ISettingsCategory.h"
#include "ISettingsContainer.h"
#include "ISettingsSection.h"
#include "Styling/AppStyle.h"
#include "Widgets/Views/STreeView.h"

using namespace UE::SettingsEditor::Private;

void SSettingsEditorCategoryTree::Construct(const FArguments& InArgs, const TSharedPtr<ISettingsContainer>& InSettingsContainer)
{
	checkf(InSettingsContainer.IsValid(), TEXT("Settings container must be valid"));
	SettingsContainer = InSettingsContainer;
	OnSelectionChanged = InArgs._OnSelectionChanged;

	ChildSlot
	.VAlign(VAlign_Fill)
	[
		SAssignNew(TreeView, STreeView<TSharedPtr<FTreeItem>>)
		.TreeViewStyle(&FAppStyle::Get().GetWidgetStyle<FTableViewStyle>("SimpleListView"))
		.SelectionMode(ESelectionMode::Single)
		.ClearSelectionOnClick(false)
		.TreeItemsSource(&TreeItemsSource)
		.OnIsSelectableOrNavigable(this, &SSettingsEditorCategoryTree::OnIsSelectable)
		.OnGenerateRow(this, &SSettingsEditorCategoryTree::OnTreeGenerateRow) 
		.OnGetChildren(this, &SSettingsEditorCategoryTree::OnTreeGetChildren)
		.OnSelectionChanged(this, &SSettingsEditorCategoryTree::OnTreeSelectionChanged)
	];

	UpdateTreeItemsSource();

	if (InArgs._InitialSelection.IsValid())
	{
		SetSelectionInternal(InArgs._InitialSelection);
	}
	else
	{
		SetSelectionInternal(/** Section */nullptr);
	}

	SettingsContainer->OnCategoryModified().AddSP(this, &SSettingsEditorCategoryTree::OnCategoryModified);
	FInternationalization::Get().OnCultureChanged().AddSP(this, &SSettingsEditorCategoryTree::OnCultureChanged);
}

SSettingsEditorCategoryTree::~SSettingsEditorCategoryTree()
{
	if (SettingsContainer)
	{
		SettingsContainer->OnCategoryModified().RemoveAll(this);
	}
	FInternationalization::Get().OnCultureChanged().RemoveAll(this);
}

void SSettingsEditorCategoryTree::SetSelection(TSharedPtr<ISettingsSection> InSection)
{
	if (TreeView->GetNumItemsSelected() == 1 && GetSelectedSection() == InSection)
	{
		return;
	}

	SetSelectionInternal(InSection);
}

TSharedPtr<ISettingsSection> SSettingsEditorCategoryTree::GetSelectedSection() const
{
	TArray<TSharedPtr<FTreeItem>> SelectedItems = TreeView->GetSelectedItems();
	return SelectedItems.IsValidIndex(0) && SelectedItems[0].IsValid() ? SelectedItems[0]->Section : nullptr;
}

void SSettingsEditorCategoryTree::ForEachSection(TFunctionRef<bool(const TSharedPtr<ISettingsSection>&)> InFunctor)
{
	for (const TSharedPtr<FTreeItem>& TreeItem : TreeItemsSource)
	{
		if (!TreeItem.IsValid() || !TreeItem->Category.IsValid())
		{
			continue;
		}

		// Load children once if needed
		if (TreeItem->Children.IsEmpty())
		{
			OnTreeGetChildren(TreeItem, TreeItem->Children);
		}

		for (const TSharedPtr<FTreeItem>& ChildItem : TreeItem->Children)
		{
			if (!ChildItem.IsValid() || !ChildItem->Section.IsValid())
			{
				continue;
			}
				
			if (!InFunctor(ChildItem->Section))
			{
				return;
			}
		}
	}
}

TSharedRef<ITableRow> SSettingsEditorCategoryTree::OnTreeGenerateRow(TSharedPtr<FTreeItem> InItem, const TSharedRef<STableViewBase>& InOwnerTable)
{
	return SNew(SSettingsEditorCategoryItem, InOwnerTable, InItem.ToSharedRef());
}

void SSettingsEditorCategoryTree::OnTreeGetChildren(TSharedPtr<FTreeItem> InItem, TArray<TSharedPtr<FTreeItem>>& OutChildren)
{
	// Only process if Category is valid and Section is not yet set
	if (!InItem.IsValid() || !InItem->Category.IsValid() || InItem->Section.IsValid())
	{
		return;
	}
	
	// Children have already been loaded previously
	if (!InItem->Children.IsEmpty())
	{
		OutChildren = InItem->Children;
		return;
	}

	TArray<TSharedPtr<ISettingsSection>> CategorySections;
	InItem->Category->GetSections(CategorySections);

	// sort the sections alphabetically
	struct FSectionSortPredicate
	{
		FORCEINLINE bool operator()(const TSharedPtr<ISettingsSection>& InSectionA, const TSharedPtr<ISettingsSection>& InSectionB) const
		{
			if (!InSectionA.IsValid() && !InSectionB.IsValid())
			{
				return false;
			}

			if (InSectionA.IsValid() != InSectionB.IsValid())
			{
				return InSectionB.IsValid();
			}

			return (InSectionA->GetDisplayName().CompareTo(InSectionB->GetDisplayName()) < 0);
		}
	};

	CategorySections.Sort(FSectionSortPredicate());

	InItem->Children.Reserve(CategorySections.Num());
	for (const TSharedPtr<ISettingsSection>& CategorySection : CategorySections)
	{
		if (!CategorySection.IsValid())
		{
			continue;
		}
		
		TSharedPtr<FTreeItem> NewTreeItem = MakeShared<FTreeItem>();
		NewTreeItem->Category = InItem->Category;
		NewTreeItem->Section = CategorySection;
		InItem->Children.Add(NewTreeItem);
	}

	OutChildren = InItem->Children;
}

void SSettingsEditorCategoryTree::OnTreeSelectionChanged(TSharedPtr<FTreeItem> InItem, ESelectInfo::Type InSelectInfo)
{
	if (!InItem.IsValid())
	{
		return;
	}

	OnSelectionChanged.ExecuteIfBound(InItem->Section);
}

bool SSettingsEditorCategoryTree::OnIsSelectable(TSharedPtr<UE::SettingsEditor::Private::FTreeItem> InItem)
{
	if (!InItem.IsValid())
	{
		return false;
	}

	// "All settings" option will not have any valid section or category, the rest should have both
	const bool bValidSection = InItem->Category.IsValid() && InItem->Section.IsValid();
	const bool bAllSections = !InItem->Category.IsValid() && !InItem->Section.IsValid();
	return bValidSection || bAllSections;
}

void SSettingsEditorCategoryTree::UpdateTreeItemsSource()
{
	TreeItemsSource.Reset();

	if (!SettingsContainer.IsValid())
	{
		return;
	}

	TArray<TSharedPtr<ISettingsCategory>> Categories;
	SettingsContainer->GetCategories(Categories);

	TreeItemsSource.Reserve(Categories.Num() + 1);

	// All section entry
	{
		TSharedPtr<FTreeItem> AllTreeItem = MakeShared<FTreeItem>();
		AllTreeItem->Category = nullptr;
		AllTreeItem->Section = nullptr;
		TreeItemsSource.Add(AllTreeItem);
	}

	for (const TSharedPtr<ISettingsCategory>& Category : Categories)
	{
		if (!Category.IsValid())
		{
			continue;
		}

		TSharedPtr<FTreeItem> NewTreeItem = MakeShared<FTreeItem>();
		NewTreeItem->Category = Category;
		NewTreeItem->Section = nullptr;
		TreeItemsSource.Add(NewTreeItem);
	}

	TreeView->RequestTreeRefresh();

	// Expand everything to match legacy behavior
	for (const TSharedPtr<FTreeItem>& Item : TreeItemsSource)
	{
		TreeView->SetItemExpansion(Item.ToSharedRef(), true);
	}
}

void SSettingsEditorCategoryTree::OnCultureChanged()
{
	RefreshTreeItems();
}

void SSettingsEditorCategoryTree::OnCategoryModified(const FName& InCategory)
{
	if (CategoryUpdateDelegate.IsValid())
	{
		return;
	}

	CategoryUpdateDelegate = FTSTicker::GetCoreTicker().AddTicker(
		FTickerDelegate::CreateSPLambda(this, 
			[this](float)->bool
			{
				CategoryUpdateDelegate.Reset();
				RefreshTreeItems();
				return false; // stop
			}
		)
	);
}

void SSettingsEditorCategoryTree::RefreshTreeItems()
{
	const TSharedPtr<ISettingsSection> SelectedSection = GetSelectedSection();
	UpdateTreeItemsSource();
	SetSelectionInternal(SelectedSection);
}

void SSettingsEditorCategoryTree::SetSelectionInternal(TSharedPtr<ISettingsSection> InSection)
{
	// Select "All" item
	if (!InSection.IsValid())
	{
		if (TreeItemsSource.IsValidIndex(0))
		{
			TreeView->SetSelection(TreeItemsSource[0], ESelectInfo::Direct);
		}
		return;
	}

	const TSharedPtr<ISettingsCategory> SettingsCategory = InSection->GetCategory().Pin();
	for (const TSharedPtr<FTreeItem>& TreeItem : TreeItemsSource)
	{
		if (!TreeItem.IsValid())
		{
			continue;
		}

		if (TreeItem->Category == SettingsCategory)
		{
			for (const TSharedPtr<FTreeItem>& ChildItem : TreeItem->Children)
			{
				if (!ChildItem.IsValid())
				{
					continue;
				}
				
				if (ChildItem->Section == InSection)
				{
					TreeView->SetSelection(ChildItem, ESelectInfo::Direct);
					return;
				}
			}
		}
	}
}
