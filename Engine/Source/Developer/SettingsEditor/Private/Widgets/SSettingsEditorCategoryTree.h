// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Ticker.h"
#include "SSettingsEditorCategoryItem.h"
#include "Widgets/SCompoundWidget.h"

class ISettingsContainer;
class ISettingsCategory;
class ISettingsSection;
class ITableRow;
class STableViewBase;
namespace ESelectInfo { enum Type : int; }
template <typename ItemType> class STreeView;

/**
 * Tree view that displays all the editor settings categories
 */
class SSettingsEditorCategoryTree : public SCompoundWidget
{
public:
	DECLARE_DELEGATE_OneParam(FOnSelectionChanged, TSharedPtr<ISettingsSection> /** Section */)

	SLATE_BEGIN_ARGS(SSettingsEditorCategoryTree)
	{}
		/** Delegate for when selection has changed */
		SLATE_EVENT(FOnSelectionChanged, OnSelectionChanged)
		/** Initial selection */
		SLATE_ARGUMENT(TSharedPtr<ISettingsSection>, InitialSelection)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedPtr<ISettingsContainer>& InSettingsContainer);
	virtual ~SSettingsEditorCategoryTree() override;
	
	void SetSelection(TSharedPtr<ISettingsSection> InSection);
	TSharedPtr<ISettingsSection> GetSelectedSection() const;

	void ForEachSection(TFunctionRef<bool(const TSharedPtr<ISettingsSection>&)> InFunctor);

private:
	TSharedRef<ITableRow> OnTreeGenerateRow(TSharedPtr<UE::SettingsEditor::Private::FTreeItem> InItem, const TSharedRef<STableViewBase>& InOwnerTable);
	void OnTreeGetChildren(TSharedPtr<UE::SettingsEditor::Private::FTreeItem> InItem, TArray<TSharedPtr<UE::SettingsEditor::Private::FTreeItem>>& OutChildren);
	void OnTreeSelectionChanged(TSharedPtr<UE::SettingsEditor::Private::FTreeItem> InItem, ESelectInfo::Type InSelectInfo);
	bool OnIsSelectable(TSharedPtr<UE::SettingsEditor::Private::FTreeItem> InItem);
	void UpdateTreeItemsSource();

	/** Callback for when the user's culture has changed. */
	void OnCultureChanged();

	/** Callback for the modification of categories in the settings container. */
	void OnCategoryModified(const FName& InCategory);
	
	void RefreshTreeItems();
	
	void SetSelectionInternal(TSharedPtr<ISettingsSection> InSection);

	TSharedPtr<STreeView<TSharedPtr<UE::SettingsEditor::Private::FTreeItem>>> TreeView;
	TArray<TSharedPtr<UE::SettingsEditor::Private::FTreeItem>> TreeItemsSource;
	TSharedPtr<ISettingsContainer> SettingsContainer;
	FOnSelectionChanged OnSelectionChanged;
	FTSTicker::FDelegateHandle CategoryUpdateDelegate;
};

