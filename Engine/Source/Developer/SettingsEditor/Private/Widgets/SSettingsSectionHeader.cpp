// Copyright Epic Games, Inc. All Rights Reserved.


#include "Widgets/SSettingsSectionHeader.h"
#include "ISettingsCategory.h"
#include "Widgets/SBoxPanel.h"
#include "Framework/Notifications/NotificationManager.h"
#include "HAL/PlatformFileManager.h"
#include "Misc/MessageDialog.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Text/STextBlock.h"
#include "Styling/AppStyle.h"
#include "ISourceControlModule.h"
#include "SSettingsEditorCheckoutNotice.h"
#include "DesktopPlatformModule.h"
#include "IDetailsView.h"
#include "Widgets/Views/SExpanderArrow.h"
#include "Widgets/Views/STableRow.h"
#include "HAL/IConsoleManager.h"

#define LOCTEXT_NAMESPACE "SSettingsEditor"

// Workaround to hide Set As Default button until there's a better way to determine if config file is cooked
// Currently the only way to know is to check if MakeDefaultConfigFileWritable returns false, but that can't happen before the button is clicked
bool bHideSetAsDefaultButton = false;
FAutoConsoleVariableRef CVarHideSetAsDefaultButton(TEXT("SettingsEditor.HideSetAsDefaultButton"), bHideSetAsDefaultButton, TEXT("Hide the Settings Editor button to save to default config."));

void SSettingsSectionHeader::ShowNotification(const FText& InText, SNotificationItem::ECompletionState InCompletionState, FString InNavigateTo)
{
	FNotificationInfo Notification(InText);
	Notification.ExpireDuration = 10.f;
	Notification.bFireAndForget = true;
	Notification.bUseSuccessFailIcons = false;

	if (!InNavigateTo.IsEmpty())
	{
		Notification.Hyperlink = FSimpleDelegate::CreateLambda([InNavigateTo]()
			{
				IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
				if (PlatformFile.FileExists(*InNavigateTo) || PlatformFile.DirectoryExists(*InNavigateTo))
				{
					// Navigate in explorer
					FPlatformProcess::ExploreFolder(*InNavigateTo);
				}
			}
		);
		Notification.HyperlinkText = LOCTEXT("NavigateToFile", "Navigate to file");
	}

	FSlateNotificationManager::Get().AddNotification(Notification)->SetCompletionState(InCompletionState);
}

void SSettingsSectionHeader::Construct(const FArguments& InArgs, const UObject* InSettingsObject, ISettingsEditorModelPtr InModel, TSharedPtr<IDetailsView> InDetailsView, const TSharedPtr<ITableRow>& InTableRow)
{
	Model = InModel;
	SettingsObject = MakeWeakObjectPtr(const_cast<UObject*>(InSettingsObject));
	SettingsSection = Model->GetSectionFromSectionObject(InSettingsObject);
	DetailsView = InDetailsView;
	TableRow = InTableRow;

	Model->OnSelectionChanged().AddSP(this, &SSettingsSectionHeader::OnSettingsSelectionChanged);

	// Create the watcher widget for the default config file (checks file status / SCC state)
	FileWatcherWidget =
		SNew(SSettingsEditorCheckoutNotice)
		.Visibility(this, &SSettingsSectionHeader::GetCheckoutNoticeVisibility)
		.OnFileProbablyModifiedExternally(this, &SSettingsSectionHeader::HandleCheckoutNoticeFileProbablyModifiedExternally)
		.ConfigFilePath(this, &SSettingsSectionHeader::HandleCheckoutNoticeConfigFilePath);


	ChildSlot
	.Padding(FMargin(0.0f, 8.0f, 0.0f, 5.0f))
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.FillHeight(1.0f)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.Padding(2.0f, 2.0f, 2.0f, 2.0f)
				.AutoWidth()
				[
					SNew(SExpanderArrow, InTableRow)
				]
				+ SHorizontalBox::Slot()// category title
				[
					SNew(STextBlock)
					.Font(FAppStyle::GetFontStyle("SettingsEditor.CatgoryAndSectionFont"))
					.Text(GetSettingsBoxTitleText())
					.OverflowPolicy(ETextOverflowPolicy::MiddleEllipsis)
				]
			]
			+SVerticalBox::Slot()
			.Padding(FMargin(0.f, 8.f, 0.f, 0.f))
			.AutoHeight()
			.MinHeight(TAttribute<float>(this, &SSettingsSectionHeader::GetDescriptionRowMinHeight))
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.FillWidth(1.0f)
				.VAlign(VAlign_Fill)
				.Padding(20.0f, 0.0f, 0.0f, 0.0f)
				[
					SNew(SBorder)
					.Padding(2.f, 0.f, 0.f, 0.f)
					.VAlign(VAlign_Center)
					.BorderImage(FAppStyle::GetBrush("NoBorder"))
					.Visibility(this, &SSettingsSectionHeader::GetContentVisibility)
					[
						// category description
						SNew(STextBlock)
						.ColorAndOpacity(FSlateColor::UseSubduedForeground())
						.Text(GetSettingsBoxDescriptionText())
						.OverflowPolicy(ETextOverflowPolicy::Ellipsis)
					]
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Center)
				.Padding(16.0f, 0.0f, 0.0f, 0.0f)
				[
					SNew(SHorizontalBox)
					.Visibility(this, &SSettingsSectionHeader::GetButtonRowVisibility)
					+SHorizontalBox::Slot()
					[
						// set as default button
						SNew(SButton)
						.Visibility(this, &SSettingsSectionHeader::HandleSetAsDefaultButtonVisibility)
						.IsEnabled(this, &SSettingsSectionHeader::HandleSetAsDefaultButtonEnabled)
						.OnClicked(this, &SSettingsSectionHeader::HandleSetAsDefaultButtonClicked)
						.Text(LOCTEXT("SaveDefaultsButtonText", "Set as Default"))
						.ToolTipText(LOCTEXT("SaveDefaultsButtonTooltip", "Save the values below as the new default settings"))
					]
					+SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(8.0f, 0.0f, 0.0f, 0.0f)
					[
						// import button
						SNew(SButton)
						.IsEnabled(this, &SSettingsSectionHeader::HandleImportButtonEnabled)
						.OnClicked(this, &SSettingsSectionHeader::HandleImportButtonClicked)
						.Text(LOCTEXT("ImportButtonText", "Import..."))
						.ToolTipText(LOCTEXT("ImportButtonTooltip", "Import these settings from a file on your computer"))
					]
					+SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(8.0f, 0.0f, 0.0f, 0.0f)
					[
						// export button
						SNew(SButton)
						.IsEnabled(this, &SSettingsSectionHeader::HandleExportButtonEnabled)
						.OnClicked(this, &SSettingsSectionHeader::HandleExportButtonClicked)
						.Text(LOCTEXT("ExportButtonText", "Export..."))
						.ToolTipText(LOCTEXT("ExportButtonTooltip", "Export these settings to a file on your computer"))
					]
					+SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(8.0f, 0.0f, 0.0f, 0.0f)
					[
						// reset defaults button
						SNew(SButton)
						.Visibility(this, &SSettingsSectionHeader::HandleSetAsDefaultButtonVisibility)
						.IsEnabled(this, &SSettingsSectionHeader::HandleResetToDefaultsButtonEnabled)
						.OnClicked(this, &SSettingsSectionHeader::HandleResetDefaultsButtonClicked)
						.Text(LOCTEXT("ResetDefaultsButtonText", "Reset to Defaults"))
						.ToolTipText(LOCTEXT("ResetDefaultsButtonTooltip", "Reset the settings below to their default values"))
					]
				]
			]
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0.0f, 8.0f, 0.0f, 0.0f)
		[
			FileWatcherWidget.ToSharedRef()
		]
	];

}

FText SSettingsSectionHeader::GetSettingsBoxTitleText() const
{
	if(SettingsSection.IsValid())
	{
		static const FText TitleFmt = FText::FromString(TEXT("{0} - {1}"));
		return FText::Format(TitleFmt, SettingsSection->GetCategory().Pin()->GetDisplayName(), SettingsSection->GetDisplayName());
	}

	return FText::GetEmpty();
}

FText SSettingsSectionHeader::GetSettingsBoxDescriptionText() const
{
	if(SettingsSection.IsValid())
	{
		return SettingsSection->GetDescription();
	}

	return FText::GetEmpty();
}

EVisibility SSettingsSectionHeader::GetButtonRowVisibility() const
{
	const EVisibility DefaultVisibility = GetContentVisibility();

	if (DefaultVisibility != EVisibility::Visible)
	{
		return DefaultVisibility;
	}
	else
	{
		return DetailsView.Pin()->HasActiveSearch() ? EVisibility::Collapsed : EVisibility::Visible;
	}
}

EVisibility SSettingsSectionHeader::GetContentVisibility() const
{
	return TableRow.IsValid() && TableRow.Pin()->IsItemExpanded() ? EVisibility::Visible : EVisibility::Collapsed;
}

FReply SSettingsSectionHeader::HandleExportButtonClicked()
{
	if(SettingsSection.IsValid())
	{
		if(LastExportDir.IsEmpty())
		{
			LastExportDir = FPaths::GetPath(GEditorPerProjectIni);
		}

		FString DefaultFileName = FString::Printf(TEXT("%s Backup %s.ini"), *SettingsSection->GetDisplayName().ToString(), *FDateTime::Now().ToString(TEXT("%Y-%m-%d %H%M%S")));
		TArray<FString> OutFiles;

		TSharedPtr<SWindow> ParentWindow = FSlateApplication::Get().FindWidgetWindow(AsShared());
		void* ParentWindowHandle = (ParentWindow.IsValid() && ParentWindow->GetNativeWindow().IsValid()) ? ParentWindow->GetNativeWindow()->GetOSWindowHandle() : nullptr;

		if(FDesktopPlatformModule::Get()->SaveFileDialog(ParentWindowHandle, LOCTEXT("ExportSettingsDialogTitle", "Export settings...").ToString(), LastExportDir, DefaultFileName, TEXT("Config files (*.ini)|*.ini"), EFileDialogFlags::None, OutFiles))
		{
			if(SettingsSection->Export(OutFiles[0]))
			{
				ShowNotification(LOCTEXT("ExportSettingsSuccess", "Export settings succeeded"), SNotificationItem::CS_Success, OutFiles[0]);
			}
			else
			{
				ShowNotification(LOCTEXT("ExportSettingsFailure", "Export settings failed"), SNotificationItem::CS_Fail);
			}
		}
	}

	return FReply::Handled();
}


float SSettingsSectionHeader::GetDescriptionRowMinHeight() const
{
	if (GetContentVisibility() == EVisibility::Visible)
	{
		return 22.f; // so that we keep a constant height even if the buttons are collapsed
	}
	else
	{
		return 0.f;
	}
}


bool SSettingsSectionHeader::HandleExportButtonEnabled() const
{
	if(SettingsSection.IsValid())
	{
		return SettingsSection->CanExport();
	}

	return false;
}


FReply SSettingsSectionHeader::HandleImportButtonClicked()
{
	if(SettingsSection.IsValid())
	{
		TArray<FString> OutFiles;

		TSharedPtr<SWindow> ParentWindow = FSlateApplication::Get().FindWidgetWindow(AsShared());
		void* ParentWindowHandle = (ParentWindow.IsValid() && ParentWindow->GetNativeWindow().IsValid()) ? ParentWindow->GetNativeWindow()->GetOSWindowHandle() : nullptr;

		if(FDesktopPlatformModule::Get()->OpenFileDialog(ParentWindowHandle, LOCTEXT("ImportSettingsDialogTitle", "Import settings...").ToString(), FPaths::GetPath(GEditorPerProjectIni), TEXT(""), TEXT("Config files (*.ini)|*.ini"), EFileDialogFlags::None, OutFiles))
		{
			if(SettingsSection->Import(OutFiles[0]) && SettingsSection->Save())
			{
				ShowNotification(LOCTEXT("ImportSettingsSuccess", "Import settings succeeded"), SNotificationItem::CS_Success);
			}
			else
			{
				ShowNotification(LOCTEXT("ImportSettingsFailure", "Import settings failed"), SNotificationItem::CS_Fail);
			}
		}
	}

	return FReply::Handled();
}


bool SSettingsSectionHeader::HandleImportButtonEnabled() const
{
	if(SettingsSection.IsValid())
	{
		bool CanImport = SettingsSection->CanEdit() && SettingsSection->CanImport();

		if (SettingsObject->GetClass()->HasAnyClassFlags(CLASS_DefaultConfig))
		{
			CanImport &= !IsDefaultConfigCheckOutNeeded();
		}
		return CanImport;
	}

	return false;
}


/**
* Gets the absolute path to the Default.ini for the specified object.
*
* @return The path to the file.
*/
FString SSettingsSectionHeader::GetDefaultConfigFilePath() const
{
	FString RelativeConfigFilePath = SettingsObject->GetDefaultConfigFilename();
	return FPaths::ConvertRelativePathToFull(RelativeConfigFilePath);
}


/**
* Checks whether the default config file needs to be checked out for editing.
*
* @return true if the file needs to be checked out, false otherwise.
*/
bool SSettingsSectionHeader::IsDefaultConfigCheckOutNeeded(bool bForceSourceControlUpdate) const
{
	if(SettingsObject.IsValid() && SettingsObject->GetClass()->HasAnyClassFlags(CLASS_Config | CLASS_DefaultConfig))
	{
		// We can only fetch the file watcher if it's visible otherwise fallback to source control
		if (FileWatcherWidget->GetVisibility().IsVisible())
		{
			return !FileWatcherWidget->IsUnlocked();
		}
		else
		{
			return !SettingsHelpers::IsCheckedOut(GetDefaultConfigFilePath(), bForceSourceControlUpdate);
		}
	}
	else
	{
		return false;
	}
}

FReply SSettingsSectionHeader::HandleResetDefaultsButtonClicked()
{
	if(SettingsSection.IsValid())
	{
		SettingsSection->ResetDefaults();
	}

	return FReply::Handled();
}


bool SSettingsSectionHeader::HandleResetToDefaultsButtonEnabled() const
{
	if(SettingsSection.IsValid())
	{
		return (SettingsSection->CanEdit() && SettingsSection->CanResetDefaults());
	}

	return false;
}

EVisibility SSettingsSectionHeader::HandleSetAsDefaultButtonVisibility() const
{
	return (!bHideSetAsDefaultButton && (SettingsSection.IsValid() && SettingsSection->CanSaveDefaults())) ? EVisibility::Visible : EVisibility::Collapsed;
}

FReply SSettingsSectionHeader::HandleSetAsDefaultButtonClicked()
{
	if(SettingsSection.IsValid())
	{
		if(FMessageDialog::Open(EAppMsgType::YesNo, LOCTEXT("SaveAsDefaultUserConfirm", "Are you sure you want to update the default settings?")) != EAppReturnType::Yes)
		{
			return FReply::Handled();
		}

		bool FileNeedToBeAddedToSourceControl = false;
		FText SaveAsDefaultNeedsAddMessage = LOCTEXT("SaveAsDefaultNeedsAddMessage", "The default configuration file for these settings is currently not under revision control. Would you like to add it to revision control?");
		FString DefaultConfigFilePath = GetDefaultConfigFilePath();
		
		if (FPlatformFileManager::Get().GetPlatformFile().FileExists(*DefaultConfigFilePath))
		{
			if (IsDefaultConfigCheckOutNeeded(true))
			{
				if (ISourceControlModule::Get().IsEnabled())
				{
					FText DisplayMessage;

					if (SettingsHelpers::IsSourceControlled(DefaultConfigFilePath))
					{
						DisplayMessage = LOCTEXT("SaveAsDefaultNeedsCheckoutMessage", "The default configuration file for these settings is currently not checked out. Would you like to check it out from revision control?");
					}
					else
					{
						DisplayMessage = SaveAsDefaultNeedsAddMessage;
					}

					if (FMessageDialog::Open(EAppMsgType::YesNo, DisplayMessage) == EAppReturnType::Yes)
					{
						if (!CheckOutOrAddDefaultConfigFile())
						{
							if (FMessageDialog::Open(EAppMsgType::YesNo, LOCTEXT("SaveAsDefaultsSourceControlOperationFailed", "The revision control operation failed. Would you like to make it writable?")) == EAppReturnType::Yes)
							{
								MakeDefaultConfigFileWritable();
							}
							else
							{
								return FReply::Handled();
							}							
						}
					}
				}
				else
				{
					if (FMessageDialog::Open(EAppMsgType::YesNo, LOCTEXT("SaveAsDefaultsIsReadOnlyMessage", "The default configuration file for these settings is not currently writable. Would you like to make it writable?")) == EAppReturnType::Yes)
					{
						MakeDefaultConfigFileWritable();
					}
					else
					{
						return FReply::Handled();
					}
				}
			}
		}
		else
		{
			if (ISourceControlModule::Get().IsEnabled())
			{
				FileNeedToBeAddedToSourceControl = true;
			}
		}

		SettingsSection->SaveDefaults();

		if (FileNeedToBeAddedToSourceControl)
		{
			if (FMessageDialog::Open(EAppMsgType::YesNo, SaveAsDefaultNeedsAddMessage) == EAppReturnType::Yes)
			{
				if (!CheckOutOrAddDefaultConfigFile(true))
				{
					FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("SaveAsDefaultsSourceControlFailedAddManually", "The revision control operation failed. You will need to add it manually"));
					return FReply::Handled();
				}
			}
		}

		FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("SaveAsDefaultsSucceededMessage", "The default configuration file for these settings was updated successfully. \n\nIf checked into revision control this would affect other developers."));
	}

	return FReply::Handled();
}


bool SSettingsSectionHeader::HandleSetAsDefaultButtonEnabled() const
{
	if(SettingsSection.IsValid())
	{
		return SettingsSection->CanSaveDefaults();
	}

	return false;
}

/**
* Checks out the default configuration file for the currently selected settings object.
*
* @return true if the check-out succeeded, false otherwise.
*/
bool SSettingsSectionHeader::CheckOutOrAddDefaultConfigFile(bool bForceSourceControlUpdate)
{
	if(!SettingsObject.IsValid())
	{
		return false;
	}

	// check out configuration file
	FText ErrorMessage;
	FString AbsoluteConfigFilePath = GetDefaultConfigFilePath();

	if (!SettingsHelpers::CheckOutOrAddFile(AbsoluteConfigFilePath, bForceSourceControlUpdate, false, &ErrorMessage))
	{ 
		// show errors, if any
		if (!ErrorMessage.IsEmpty())
		{
			FMessageDialog::Open(EAppMsgType::Ok, ErrorMessage);
		}

		return false;
	}

	return true;
}

/**
* Makes the default configuration file for the currently selected settings object writable.
*
* @return true if it was made writable, false otherwise.
*/
bool SSettingsSectionHeader::MakeDefaultConfigFileWritable()
{
	if(!SettingsObject.IsValid())
	{
		return false;
	}

	FString AbsoluteConfigFilePath = GetDefaultConfigFilePath();

	return SettingsHelpers::MakeWritable(AbsoluteConfigFilePath, true);
}

/** Returns the config file name currently being edited. */
FString SSettingsSectionHeader::HandleCheckoutNoticeConfigFilePath() const
{
	if(SettingsObject.IsValid())
	{
		if (SettingsObject->GetClass()->HasAnyClassFlags(CLASS_DefaultConfig))
		{
			return GetDefaultConfigFilePath();
		}
		else if (SettingsObject->GetClass()->HasAnyClassFlags(CLASS_Config))
		{
			return SettingsObject->GetClass()->GetConfigName();
		}
	}

	return FString();
}

/** Reloads the configuration object. */
void SSettingsSectionHeader::HandleCheckoutNoticeFileProbablyModifiedExternally()
{
	if(SettingsObject.IsValid() && SettingsObject->GetClass()->HasAnyClassFlags(CLASS_Config | CLASS_DefaultConfig))
	{
		SettingsObject->ReloadConfig();
	}
}

/** Callback for determining the visibility of the 'Locked' notice. */
EVisibility SSettingsSectionHeader::GetCheckoutNoticeVisibility() const
{
	const EVisibility DefaultVisibility = GetContentVisibility();

	if (DefaultVisibility != EVisibility::Visible)
	{
		return DefaultVisibility;
	}

	// Only DefaultConfig are under source control, so the checkout notice should not be visible for the other cases
	if(SettingsObject.IsValid() && SettingsObject->GetClass()->HasAnyClassFlags(CLASS_DefaultConfig) && !DetailsView.Pin()->HasActiveSearch())
	{
		return EVisibility::Visible;
	}

	return EVisibility::Collapsed;
}

void SSettingsSectionHeader::OnSettingsSelectionChanged()
{
	FileWatcherWidget->Invalidate();
}


FSettingsDetailRootObjectCustomization::FSettingsDetailRootObjectCustomization(ISettingsEditorModelPtr InModel, const TSharedRef<IDetailsView>& InDetailsView)
	: Model(InModel)
	, DetailsView(InDetailsView)
{
}

void FSettingsDetailRootObjectCustomization::Initialize()
{
	Model->OnSelectionChanged().AddSP(this, &FSettingsDetailRootObjectCustomization::OnSelectedSectionChanged);
	// Call once to ensure we have a valid section object
	OnSelectedSectionChanged();
}

TSharedPtr<SWidget> FSettingsDetailRootObjectCustomization::CustomizeObjectHeader(const FDetailsObjectSet& InRootObjectSet, const TSharedPtr<ITableRow>& InTableRow)
{
	return SNew(SSettingsSectionHeader, InRootObjectSet.RootObjects[0], Model, DetailsView.Pin(), InTableRow);
}

bool FSettingsDetailRootObjectCustomization::AreObjectsVisible(const FDetailsObjectSet& InRootObjectSet) const
{
	return !SelectedSectionObject.IsValid() || (!InRootObjectSet.RootObjects.IsEmpty() && SelectedSectionObject == InRootObjectSet.RootObjects[0]);
}

bool FSettingsDetailRootObjectCustomization::ShouldDisplayHeader(const FDetailsObjectSet& InRootObjects) const
{
	return true;
}

void FSettingsDetailRootObjectCustomization::OnSelectedSectionChanged()
{
	ISettingsSectionPtr SelectedSection = Model->GetSelectedSection();
	if(SelectedSection.IsValid())
	{
		SelectedSectionObject = SelectedSection->GetSettingsObject();
		SelectedSection->Select();
	}
	else
	{
		SelectedSectionObject = nullptr;
	}

	DetailsView.Pin()->RefreshRootObjectVisibility();
}

#undef LOCTEXT_NAMESPACE
