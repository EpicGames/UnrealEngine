// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Layout/Visibility.h"
#include "Input/Reply.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "ISettingsEditorModel.h"
#include "ISettingsSection.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "IDetailRootObjectCustomization.h"

class IDetailsView;
class SSettingsEditorCheckoutNotice;

class SSettingsSectionHeader : public SCompoundWidget
{
public:
	static void ShowNotification(const FText& InText, SNotificationItem::ECompletionState InCompletionState, FString InNavigateTo = FString());

	SLATE_BEGIN_ARGS(SSettingsSectionHeader)
	{}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const UObject* InSettingsObject, ISettingsEditorModelPtr InModel, TSharedPtr<IDetailsView> InDetailsView, const TSharedPtr<ITableRow>& InTableRow);

private:

	FText GetSettingsBoxTitleText() const;
	FText GetSettingsBoxDescriptionText() const;

	FReply HandleExportButtonClicked();


	bool HandleExportButtonEnabled() const;


	FReply HandleImportButtonClicked();
	bool HandleImportButtonEnabled() const;


	/**
	* Gets the absolute path to the Default.ini for the specified object.
	*
	* @return The path to the file.
	*/
	FString GetDefaultConfigFilePath() const;


	/**
	* Checks whether the default config file needs to be checked out for editing.
	*
	* @return true if the file needs to be checked out, false otherwise.
	*/
	bool IsDefaultConfigCheckOutNeeded(bool bForceSourceControlUpdate = false) const;

	FReply HandleResetDefaultsButtonClicked();


	bool HandleResetToDefaultsButtonEnabled() const;

	EVisibility HandleSetAsDefaultButtonVisibility() const;

	FReply HandleSetAsDefaultButtonClicked();


	bool HandleSetAsDefaultButtonEnabled() const;

	/**
	* Checks out the default configuration file for the currently selected settings object.
	*
	* @return true if the check-out succeeded, false otherwise.
	*/
	bool CheckOutOrAddDefaultConfigFile(bool bForceSourceControlUpdate = false);

	/**
	* Makes the default configuration file for the currently selected settings object writable.
	*
	* @return true if it was made writable, false otherwise.
	*/
	bool MakeDefaultConfigFileWritable();

	/** Returns the config file name currently being edited. */
	FString HandleCheckoutNoticeConfigFilePath() const;

	/** Reloads the configuration object. */
	void HandleCheckoutNoticeFileProbablyModifiedExternally();

	/** Callback for determining the visibility of the 'Locked' notice. */
	EVisibility GetCheckoutNoticeVisibility() const;

	EVisibility GetButtonRowVisibility() const;

	EVisibility GetContentVisibility() const;

	float GetDescriptionRowMinHeight() const;

	void OnSettingsSelectionChanged();
private:
	/** Watcher widget for the default config file (checks file status / SCC state). */
	TSharedPtr<SSettingsEditorCheckoutNotice> FileWatcherWidget;
	FString LastExportDir;
	ISettingsEditorModelPtr Model;
	ISettingsSectionPtr SettingsSection;
	TWeakObjectPtr<UObject> SettingsObject;
	TWeakPtr<IDetailsView> DetailsView;
	TWeakPtr<ITableRow> TableRow;
};


class FSettingsDetailRootObjectCustomization : public IDetailRootObjectCustomization
{
public:
	FSettingsDetailRootObjectCustomization(ISettingsEditorModelPtr InModel, const TSharedRef<IDetailsView>& InDetailsView);

	void Initialize();

	/** IDetailRootObjectCustomization interface */
	virtual TSharedPtr<SWidget> CustomizeObjectHeader(const FDetailsObjectSet& InRootObjectSet, const TSharedPtr<ITableRow>& InTableRow) override;
	virtual bool AreObjectsVisible(const FDetailsObjectSet& InRootObjectSet) const override;
	virtual bool ShouldDisplayHeader(const FDetailsObjectSet& InRootObjectSet) const override;
	virtual EExpansionArrowUsage GetExpansionArrowUsage() const override { return EExpansionArrowUsage::Custom; }
private:
	void OnSelectedSectionChanged();

private:
	ISettingsEditorModelPtr Model;
	TWeakObjectPtr<UObject> SelectedSectionObject;
	TWeakPtr<IDetailsView> DetailsView;

};
