// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class ISettingsContainer;
class ISettingsSection;

/**
 * Interface for settings editor view models.
 *
 * The settings editor view model stores the view state for the Settings Editor UI.
 * Instances of this interface can be passed to Settings Editors in order to provide
 * access to the settings container being added and to perform various user actions,
 * such as setting the currently selected settings section.
 */
class ISettingsEditorModel
{
public:

	/**
	 * Gets the selected settings section.
	 *
	 * @return The selected section, if any.
	 */
	virtual const TSharedPtr<ISettingsSection>& GetSelectedSection() const = 0;

	/**
	 * Gets the settings container.
	 *
	 * @return The settings container.
	 */
	virtual const TSharedRef<ISettingsContainer>& GetSettingsContainer() const = 0;

	/**
	 * Selects the specified settings section to be displayed in the editor.
	 *
	 * @param Section The section to select.
	 *
	 * @see SelectPreviousSection
	 */
	virtual void SelectSection( const TSharedPtr<ISettingsSection>& Section ) = 0;

	/**
	 * Gets the section interface for a specified section object
	 *
	 * @param SectionObject	The settings section object to get the section interface for
	 *
	 * @return The section interface for the section object or nullptr if it does not exist
	 */
	virtual TSharedPtr<ISettingsSection> GetSectionFromSectionObject(const UObject* SectionObject) const = 0;
public:

	/**
	 * Returns a delegate that is executed when the selected settings section has changed.
	 *
	 * @return The delegate.
	 */
	virtual FSimpleMulticastDelegate& OnSelectionChanged() = 0;

public:

	/** Virtual destructor. */
	virtual ~ISettingsEditorModel() = default;
};


/** Type definition for shared pointers to instances of ISettingsEditorModel. */
typedef TSharedPtr<ISettingsEditorModel> ISettingsEditorModelPtr;

/** Type definition for shared references to instances of ISettingsEditorModel. */
typedef TSharedRef<ISettingsEditorModel> ISettingsEditorModelRef;
