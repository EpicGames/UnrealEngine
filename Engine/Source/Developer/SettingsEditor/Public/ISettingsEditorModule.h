// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Delegates/DelegateCombinations.h"
#include "Modules/ModuleInterface.h"
#include "Widgets/SWidget.h"

class ISettingsContainer;
class ISettingsEditorModel;
class UDeveloperSettings;

DECLARE_DELEGATE_RetVal_OneParam(bool, FShouldRegisterSettingsDelegate , UDeveloperSettings* /*Settings*/);

/**
 * Interface for settings editor modules.
 */
class ISettingsEditorModule
	: public IModuleInterface
{
public:

	/**
	 * Creates a settings editor widget.
	 *
	 * @param Model The view model.
	 * @return The new widget.
	 * @see CreateModel
	 */
	virtual TSharedRef<SWidget> CreateEditor( const TSharedRef<ISettingsEditorModel>& Model ) = 0;

	/**
	 * Creates a view model for the settings editor widget.
	 *
	 * @param SettingsContainer The settings container.
	 * @return The controller.
	 * @see CreateEditor
	 */
	virtual TSharedRef<ISettingsEditorModel> CreateModel( const TSharedRef<ISettingsContainer>& SettingsContainer ) = 0;

	/**
	 * Called when the settings have been changed such that an application restart is required for them to be fully applied
	 */
	virtual void OnApplicationRestartRequired() = 0;

	/**
	 * Set the delegate that should be called when a setting editor needs to restart the application
	 *
	 * @param InRestartApplicationDelegate The new delegate to call
	 */
	virtual void SetRestartApplicationCallback( FSimpleDelegate InRestartApplicationDelegate ) = 0;

	/**
	 * Set the delegate that should be called when a setting editor checks whether a settings object should be registered.
	 * 
	 * @param InShouldRegisterSettingDelegate The new delegate to call.
	 */
	virtual void SetShouldRegisterSettingCallback(FShouldRegisterSettingsDelegate InShouldRegisterSettingDelegate) = 0;

	/**
	 * Registers any pending auto-discovered settings.
	 *
	 * @param bForce Forces the registration of settings even if there is no active settings editor.
	 */
	virtual void UpdateSettings(bool bForce = false) = 0;

	struct FSearchTerm
	{
		/** Used to show this search term in the UI */
		FText Label;

		/** Tooltip text to show on the filter button */
		FText Tooltip;

		/** Name of the style set to query icon from */
		FName IconStyleSetName;

		/** Name of the resource to use for the icon */
		FName IconStyleName;

		/** Results that should be listed and suggested for this search term */
		TSet<FString> Values;

		/** Only show this search term and filter in those sections */
		TSet<FName> AllowedSections;

		/** Do not show this search term and filter in those sections */
		TSet<FName> DisallowedSections;

		/** Show/hide filter toggle button in the UI or only show suggestions */
		bool bShowFilter = true;

		/** Returns whether this search term is allowed in a section */
		bool PassesSectionFilter(FName InSection) const
		{
			if (DisallowedSections.Contains(InSection))
			{
				return false;
			}

			if (AllowedSections.IsEmpty() || AllowedSections.Contains(InSection))
			{
				return true;
			}

			return false;
		}
	};

	/**
	 * Registers a search term for a specific container and key, does nothing if it already exists
	 * @param InContainerName The context for this search term (i.e. project/editor)
	 * @param InKey The key used to show filter/suggestions values (i.e. cvar/keybind)
	 * @return bool whether the search term is registered
	 */
	virtual bool RegisterSearchTerm(FName InContainerName, const FString& InKey) = 0;

	/** 
	 * Delegate called when new search terms have been registered
	 */
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnSearchTermsChanged, FName /** InContainerName */, const FString& /** InKey */)
	virtual FOnSearchTermsChanged::RegistrationType& OnSearchTermsChanged() = 0;

	/**
	 * Finds existing search term for a specific container and key
	 * @param InContainerName The context for this search term (i.e. project/editor)
	 * @param InKey The key used to show filter/suggestions values (i.e. cvar/keybind)
	 * @return FSearchTerm* The search term for this container and key if any
	 */
	virtual FSearchTerm* FindSearchTerm(FName InContainerName, const FString& InKey) = 0;
	
	/**
	 * Finds existing search term for a specific container and key (const version)
	 * @param InContainerName The context for this search term (i.e. project/editor)
	 * @param InKey The key used to show filter/suggestions values (i.e. cvar/keybind)
	 * @return const FSearchTerm* The search term for this container and key if any
	 */
	virtual const FSearchTerm* FindSearchTerm(FName InContainerName, const FString& InKey) const = 0;

	/**
	 * Finds existing search terms for a specific container
	 * @param InContainerName The context for this search term (i.e. project/editor)
	 * @return TMap<FString, FSearchTerm>* The search terms for this container if any
	 */
	virtual const TMap<FString, FSearchTerm>* FindSearchTerms(FName InContainerName) const = 0;

	/** Virtual destructor. */
	virtual ~ISettingsEditorModule() override = default;
};
