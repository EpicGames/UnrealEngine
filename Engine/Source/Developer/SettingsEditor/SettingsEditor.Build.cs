// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class SettingsEditor : ModuleRules
{
	public SettingsEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[] {
				"Core",
			}
		);

		PublicIncludePathModuleNames.AddRange(
			new string[] {
				"Settings",
			}
		);

		PrivateDependencyModuleNames.AddRange(
            new string[] {
				"Analytics",
                "CoreUObject",
				"DesktopPlatform",
				"DeveloperSettings",
				"EditorWidgets",
				"Engine",
                "InputCore",
				"PropertyEditor",
				"SharedSettingsWidgets",
				"Slate",
				"SlateCore",
				"SourceControl",
				"ToolMenus",
				"ToolWidgets",
            }
        );
	}
}
